// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using UnrealBuildTool;
using EpicGames.Core;

public class Launch : ModuleRules
{
	[ConfigFile(ConfigHierarchyType.Engine, "/Script/IOSRuntimeSettings.IOSRuntimeSettings")]
	bool bEnableIOSCrashDump = false;

	protected virtual bool EnableIOSCrashDump
	{
		get
		{
			ConfigCache.ReadSettings(DirectoryReference.FromFile(Target.ProjectFile), Target.Platform, this);
			return bEnableIOSCrashDump;
		}
	}
	
	protected virtual bool bAllowSwiftMain => Target.IsInPlatformGroup(UnrealPlatformGroup.Apple) && Target.Type != TargetType.Editor;
	
	public Launch(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateIncludePathModuleNames.AddRange(new string[] {
				"AutomationController",
				"AutomationTest",
				"ProfileVisualizer",
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"MoviePlayer",
				"MoviePlayerProxy",
				"Networking",
				"PakFile",
				"Projects",
				"RenderCore",
				"RHI",
				"SandboxFile",
				"Serialization",
				"ApplicationCore",
				"Slate",
				"SlateCore",
				"Sockets",
				"TraceLog",
				"Overlay",
				"PreLoadScreen",
				"InstallBundleManager",
			});

		// Set a macro allowing us to switch between debuggame/development configuration
		if (Target.Configuration == UnrealTargetConfiguration.DebugGame)
		{
			PrivateDefinitions.Add("UE_BUILD_DEVELOPMENT_WITH_DEBUGGAME=1");
		}
		else
		{
			PrivateDefinitions.Add("UE_BUILD_DEVELOPMENT_WITH_DEBUGGAME=0");
		}

		// Enable the LauncherCheck module to be used for platforms that support the Launcher.
		// Projects should set Target.bUseLauncherChecks in their Target.cs to enable the functionality.
		if (Target.bUseLauncherChecks &&
			((Target.Platform == UnrealTargetPlatform.Win64) ||
			(Target.Platform == UnrealTargetPlatform.Mac)))
		{
			PrivateDependencyModuleNames.Add("LauncherCheck");
			PublicDefinitions.Add("WITH_LAUNCHERCHECK=1");
		}
		else
		{
			PublicDefinitions.Add("WITH_LAUNCHERCHECK=0");
		}
		
		PublicDefinitions.Add("UE_ENABLE_IOS_CRASHDUMP=" + (EnableIOSCrashDump ? "1" : "0"));

		if (Target.Type != TargetType.Server)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
					"HeadMountedDisplay",
				"MediaUtils",
					"MRMesh",
			});

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				DynamicallyLoadedModuleNames.AddRange(new string[] {
					"WindowsPlatformFeatures",
				});
			}

			DynamicallyLoadedModuleNames.AddRange(new string[] {
					"AudioMixerPlatformAudioLink",
				});

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PrivateDependencyModuleNames.Add("AgilitySDK");
				DynamicallyLoadedModuleNames.AddRange(new string[] {
					"AudioMixerXAudio2",
					"AudioMixerWasapi",
				});
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				DynamicallyLoadedModuleNames.AddRange(new string[] {
					"AudioMixerCoreAudio",
				});
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
			{
				DynamicallyLoadedModuleNames.Add("AudioMixerSDL");
				PrivateDependencyModuleNames.Add("Json");
			}

			PrivateIncludePathModuleNames.AddRange(new string[] {
				"Media",
					"SlateNullRenderer",
				"SlateRHIRenderer",
			});

			DynamicallyLoadedModuleNames.AddRange(new string[] {
				"Media",
					"SlateNullRenderer",
				"SlateRHIRenderer",
			});
		}

		// UFS clients are not available in shipping builds
		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
					"NetworkFile",
					"StreamingFile",
					"AutomationWorker"
			});
		}

		DynamicallyLoadedModuleNames.AddRange(new string[] {
				"Renderer",
		});

		if (Target.bCompileAgainstEngine)
		{
			PrivateIncludePathModuleNames.AddRange(new string[] {
					"MessagingCommon",
			});

			PublicDependencyModuleNames.Add("SessionServices");

			if (Target.bBuildWithEditorOnlyData)
			{
				PrivateDependencyModuleNames.Add("DerivedDataCache");
			}
			else
			{
				PrivateIncludePathModuleNames.Add("DerivedDataCache");
			}

			// LaunchEngineLoop.cpp will still attempt to load XMPP but not all projects require it so it will silently fail unless referenced by the project's build.cs file.
			// DynamicallyLoadedModuleNames.Add("XMPP");

			DynamicallyLoadedModuleNames.AddRange(new string[] {
				"HTTP",
				"MediaAssets",
			});

			PrivateDependencyModuleNames.AddRange(new string[] {
				"ClothingSystemRuntimeNv",
				"ClothingSystemRuntimeInterface"
			});

			if (Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PrivateDependencyModuleNames.AddRange(new string[] {
					"FunctionalTesting"
				});
			}
		}

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			DynamicallyLoadedModuleNames.AddRange(new string[] {
				"ProfileVisualizer",
			});
		}

		// The engine can use AutomationController in any connfiguration besides shipping.  This module is loaded
		// dynamically in LaunchEngineLoop.cpp in non-shipping configurations
		if (Target.bCompileAgainstEngine && Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			DynamicallyLoadedModuleNames.AddRange(new string[] { "AutomationController" });
		}

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
					"SourceControl",
					"EditorFramework",
					"UnrealEd",
					"DesktopPlatform",
			});

			// ExtraModules that are loaded when WITH_EDITOR=1 is true
			DynamicallyLoadedModuleNames.AddRange(new string[] {
					"AutomationWindow",
					"OutputLog",
					"TextureCompressor",
					"MeshUtilities",
					"SourceCodeAccess",
					"EditorStyle"
			});

			if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				PrivateDependencyModuleNames.AddRange(new string[] {
					"MainFrame",
					"Settings",
				});
			}
			else
			{
				DynamicallyLoadedModuleNames.Add("MainFrame");
			}
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Android))
		{
			PrivateDependencyModuleNames.Add("OpenGLDrv");
			PrivateDependencyModuleNames.Add("AudioMixerAndroid");

			// these are, for now, only for basic android
			if (Target.Platform == UnrealTargetPlatform.Android)
			{
				DynamicallyLoadedModuleNames.Add("AndroidRuntimeSettings");
				DynamicallyLoadedModuleNames.Add("AndroidLocalNotification");
			}
		}

		if (bAllowSwiftMain)
		{
			SwiftInteropHeader = Path.Combine(ModuleDirectory, "Private/Apple/LaunchCPPToSwift.h");
		}

		if ((Target.Platform == UnrealTargetPlatform.Win64) ||
			(Target.IsInPlatformGroup(UnrealPlatformGroup.Linux) && Target.Type != TargetType.Server))
		{
			// TODO: re-enable after implementing resource tables for OpenGL.
			DynamicallyLoadedModuleNames.Add("OpenGLDrv");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			PrivateDependencyModuleNames.Add("UnixCommonStartup");
		}

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateDependencyModuleNames.Add("StorageServerClient");
			PrivateDependencyModuleNames.Add("StorageServerClientDebug");

			if (Target.Type != TargetType.Program)
			{
				PublicDependencyModuleNames.Add("CookOnTheFly");
			}
		}

	}
}
