// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidEventManager.h"

#if USE_ANDROID_EVENTS
#include "Android/AndroidApplication.h"
#include "AudioDevice.h"
#include "Misc/CallbackDevice.h"
#include <android/native_window.h>
#include <android/native_window_jni.h>
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "RenderingThread.h"
#include "UnrealEngine.h"
#include "Android/AndroidDynamicRHI.h"

DEFINE_LOG_CATEGORY(LogAndroidEvents);

FAppEventManager* FAppEventManager::sInstance = NULL;


FAppEventManager* FAppEventManager::GetInstance()
{
	if(!sInstance)
	{
		sInstance = new FAppEventManager();
	}

	return sInstance;
}

static const TCHAR* GetAppEventName(EAppEventState State)
{
	const TCHAR* Names[APP_EVENT_MAX_EVENTS] = {
		TEXT("APP_EVENT_STATE_WINDOW_CREATED"),
		TEXT("APP_EVENT_STATE_WINDOW_RESIZED"),
		TEXT("APP_EVENT_STATE_WINDOW_CHANGED"),
		TEXT("APP_EVENT_STATE_WINDOW_DESTROYED"),
		TEXT("APP_EVENT_STATE_WINDOW_REDRAW_NEEDED"),
		TEXT("APP_EVENT_STATE_ON_DESTROY"),
		TEXT("APP_EVENT_STATE_ON_PAUSE"),
		TEXT("APP_EVENT_STATE_ON_RESUME"),
		TEXT("APP_EVENT_STATE_ON_STOP"),
		TEXT("APP_EVENT_STATE_ON_START"),
		TEXT("APP_EVENT_STATE_WINDOW_LOST_FOCUS"),
		TEXT("APP_EVENT_STATE_WINDOW_GAINED_FOCUS"),
		TEXT("APP_EVENT_STATE_SAVE_STATE"),
		TEXT("APP_EVENT_STATE_APP_SUSPENDED"),
		TEXT("APP_EVENT_STATE_APP_ACTIVATED"),
		TEXT("APP_EVENT_STATE_SAFE_ZONE_UPDATED"),
		TEXT("APP_EVENT_RUN_CALLBACK"),
		};

	static_assert(APP_EVENT_MAX_EVENTS == 17, "You must update this array if more app events are added!");

	if (State == APP_EVENT_STATE_INVALID)
	{
		return TEXT("APP_EVENT_STATE_INVALID");
	}
	else if (State >= APP_EVENT_MAX_EVENTS || State < 0)
	{
		return TEXT("UnknownEAppEventStateValue");
	}
	else
	{
		return Names[State];
	}
}

void FAppEventManager::SetAppIsActive(bool bIsActive)
{
	bAppIsActive = bIsActive;
}

void FAppEventManager::ResumePausedGameThread()
{
	if (bAppIsActive || bTempRunningTriggered)
	{
		//UE_LOGF(LogAndroid, Log, "Resume_GT_Temp bypassed. %d, %d, %d", (bool)bAppIsActive, (bool)bRunning, (bool)bTempRunningTriggered);
		return;
	}

	EnqueueAppEvent(APP_EVENT_STATE_APP_ACTIVATED);

	if (EventHandlerEvent)
	{
		EventHandlerEvent->Trigger();
	}

	bTempRunningTriggered = true;

	//UE_LOGF(LogAndroid, Log, "Resume_GT_Temp done. %d, %d, %d", (bool)bAppIsActive, (bool)bRunning, (bool)bTempRunningTriggered);
}

void FAppEventManager::KeepGameThreadPaused()
{
	if (bRunning && !bAppIsActive && bTempRunningTriggered)
	{
		EnqueueAppEvent(APP_EVENT_STATE_APP_SUSPENDED);
		//UE_LOGF(LogAndroid, Log, "Pause_GT_Temp done. %d, %d, %d", (bool)bAppIsActive, (bool)bRunning, (bool)bTempRunningTriggered);
	}
	else
	{
		//UE_LOGF(LogAndroid, Log, "Pause_GT_Temp bypassed. %d, %d, %d", (bool)bAppIsActive, (bool)bRunning, (bool)bTempRunningTriggered);
	}

	bTempRunningTriggered = false;
}


void FAppEventManager::Tick()
{
	check(IsInGameThread());
	while (!Queue.IsEmpty())
	{
		FAppEventPacket Event = DequeueAppEvent();
		STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("FAppEventManager::Tick processing, %d"), int(Event.State));

		switch (Event.State)
		{
		case APP_EVENT_STATE_WINDOW_CREATED:
			FAndroidWindow::EventManagerUpdateWindowDimensions(Event.Data.WindowWidth, Event.Data.WindowHeight);
			bCreateWindow = true;
			break;
		case APP_EVENT_STATE_WINDOW_RESIZED:
			// Cache the new window's dimensions for the game thread.
			FAndroidWindow::EventManagerUpdateWindowDimensions(Event.Data.WindowWidth, Event.Data.WindowHeight);
			ExecWindowResized();
			break;
		case APP_EVENT_STATE_WINDOW_CHANGED:
			// React on device orientation/windowSize changes only when application has window
			// In case window was created this tick it should already has correct size
			// see 'UE::Jni::FGameActivity::nativeOnOrientationChanged' for event thread/game thread mismatches.
			ExecWindowResized();
		break;
		case APP_EVENT_STATE_SAVE_STATE:
			bSaveState = true; //todo android: handle save state.
			break;
		case APP_EVENT_STATE_WINDOW_DESTROYED:
			bHaveWindow = false;
			STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("APP_EVENT_STATE_WINDOW_DESTROYED, %d, %d, %d"), int(bRunning), int(bWindowInFocus), int(bHaveGame));
			break;
		case APP_EVENT_STATE_ON_START:
			//doing nothing here
			break;
		case APP_EVENT_STATE_ON_DESTROY:
			check(bHaveWindow == false);
			check(IsEngineExitRequested()); //destroy immediately. Game will shutdown.
			STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("APP_EVENT_STATE_ON_DESTROY"));
			break;
		case APP_EVENT_STATE_ON_STOP:
			bHaveGame = false;
			ReleaseMicrophone(false);
			break;
		case APP_EVENT_STATE_ON_PAUSE:
			FAndroidAppEntry::OnPauseEvent();
			bHaveGame = false;
			STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("APP_EVENT_STATE_ON_PAUSE, %d, %d, %d"), int(bRunning), int(bWindowInFocus), int(bHaveGame));
			break;
		case APP_EVENT_STATE_ON_RESUME:
			bHaveGame = true;
			STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("APP_EVENT_STATE_ON_RESUME, %d, %d, %d"), int(bRunning), int(bWindowInFocus), int(bHaveGame));
			break;

		// window focus events that follow their own hierarchy, and might or might not respect App main events hierarchy
		case APP_EVENT_STATE_WINDOW_GAINED_FOCUS: 
			bWindowInFocus = true;
			break;
		case APP_EVENT_STATE_WINDOW_LOST_FOCUS:
			bWindowInFocus = false;
			break;
		case APP_EVENT_RUN_CALLBACK:
		{
			UE_LOGF(LogAndroidEvents, Display, "Event thread callback running.");
			Event.Data.CallbackFunc();
			break;
		}
		case APP_EVENT_STATE_APP_ACTIVATED:
			bRunning = true;
			STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("Execution will be resumed!"));
			break;
		case APP_EVENT_STATE_APP_SUSPENDED:
			bRunning = false;
			STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("Execution will be paused..."));
			break;
		case APP_EVENT_STATE_SAFE_ZONE_UPDATED:
			FCoreDelegates::OnSafeFrameChangedEvent.Broadcast();
			break;
		default:
			UE_LOGF(LogAndroidEvents, Display, "Application Event : %u  not handled. ", Event.State);
		}

		if (bCreateWindow)
		{
			// wait until activity is in focus.
			if (bWindowInFocus) 
			{
				ExecWindowCreated();
				bCreateWindow = false;
				bHaveWindow = true;
				STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("ExecWindowCreated, %d, %d, %d"), int(bRunning), int(bHaveWindow), int(bHaveGame));
			}
		}
	}

	if (EmptyQueueHandlerEvent)
	{
		EmptyQueueHandlerEvent->Trigger();
	}

	if (!bRunning) // && !FAndroidMisc::UseNewWindowBehavior()) // TODO: stalling GT here could be optional
	{
		STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("FAppEventManager::Tick EventHandlerEvent Wait "));
		EventHandlerEvent->Wait();
		STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("FAppEventManager::Tick EventHandlerEvent DONE Wait "));
	}
}

void FAppEventManager::ReleaseMicrophone(bool shuttingDown)
{
	if (FModuleManager::Get().IsModuleLoaded("Voice"))
	{
		UE_LOGF(LogAndroidEvents, Log, "Android release microphone");
		if (shuttingDown)
		{
			UE_LOGF(LogAndroidEvents, Warning, "Android release microphone - Voice module shutting down - CANNOT RESUME");
		}
		FModuleManager::Get().UnloadModule("Voice", shuttingDown);
	}
}

void FAppEventManager::TriggerEmptyQueue()
{
	if (EmptyQueueHandlerEvent)
	{
		EmptyQueueHandlerEvent->Trigger();
	}
}

FAppEventManager::FAppEventManager():
	EventHandlerEvent(nullptr)
	,EmptyQueueHandlerEvent(nullptr)
	,FirstInitialized(false)
	,bCreateWindow(false)
	,bWindowInFocus(true)
	,bSaveState(false)
	,bAudioPaused(false)
	,bHaveWindow(false)
	,bHaveGame(false)
	,bRunning(false)
	,bTempRunningTriggered(false)
	,bAppIsActive(false)
{
	pthread_mutex_init(&QueueMutex, NULL);

	IConsoleVariable* CVarScale = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MobileContentScaleFactor"));
	check(CVarScale);
	CVarScale->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&FAppEventManager::OnScaleFactorChanged));

	IConsoleVariable* CVarResX = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResX"));
	if (CVarResX != nullptr)
	{
		CVarResX->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&FAppEventManager::OnScaleFactorChanged));
	}

	IConsoleVariable* CVarResY = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResY"));
	if (CVarResY != nullptr)
	{
		CVarResY->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&FAppEventManager::OnScaleFactorChanged));
	}
}

void FAppEventManager::OnScaleFactorChanged(IConsoleVariable* CVar)
{
	if ((CVar->GetFlags() & ECVF_SetByMask) == ECVF_SetByConsole)
	{
		STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("FAppEventManager::OnScaleFactorChanged with flags=0x%x"), CVar->GetFlags());
		FAppEventManager::GetInstance()->ExecWindowResized();
	}
	else
    {
		STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("NOOP FAppEventManager::OnScaleFactorChanged with flags=0x%x"), CVar->GetFlags());
    }
}

void FAppEventManager::HandleWindowCreated_EventThread(void* InWindow)
{
	if (InWindow == nullptr)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAppEventManager::HandleWindowCreated_EventThread was given a NULL window so IGNORE."));
		return;
	}

	bool AlreadyInited = FirstInitialized;

	// Make sure window will not be deleted until event is processed
	// Window could be deleted by OS while event queue stuck at game start-up phase
	FAndroidWindow::AcquireWindowRef((ANativeWindow*)InWindow);
	FAndroidWindow::SetHardwareWindow_EventThread(InWindow);

	if (!AlreadyInited)
	{
		//This cannot wait until first tick. 
		FirstInitialized = true;
	}
	EnqueueAppEvent(APP_EVENT_STATE_WINDOW_CREATED, FAppEventData((ANativeWindow*)InWindow));
}

void FAppEventManager::HandleWindowClosed_EventThread()
{
	void* ActiveWindow = FAndroidWindow::GetHardwareWindow_EventThread();
	
	if(ActiveWindow == nullptr)
	{
		return;
	}

	FAndroidWindow::ReleaseWindowRef((ANativeWindow*)ActiveWindow);
	FAndroidWindow::SetHardwareWindow_EventThread(nullptr);

	EnqueueAppEvent(APP_EVENT_STATE_WINDOW_DESTROYED);
}


void FAppEventManager::SetEventHandlerEvent(FEvent* InEventHandlerEvent)
{
	EventHandlerEvent = InEventHandlerEvent;
}

void FAppEventManager::SetEmptyQueueHandlerEvent(FEvent* InEventHandlerEvent)
{
	EmptyQueueHandlerEvent = InEventHandlerEvent;
}

void FAppEventManager::ExecWindowCreated()
{
	UE_LOGF(LogAndroidEvents, Display, "ExecWindowCreated (using new behavior %d)", FAndroidMisc::UseNewWindowBehavior());
	// When application launched while device is in sleep mode SystemResolution could be set to opposite orientation values
	// Force to update SystemResolution to current values whenever we create a new window
	FPlatformRect ScreenRect = FAndroidWindow::GetScreenRect();
	FSystemResolution::RequestResolutionChange(ScreenRect.Right - ScreenRect.Left, ScreenRect.Bottom - ScreenRect.Top, EWindowMode::Fullscreen);

	// ReInit with the new window handle
	// this code has no knowledge of multiple windows.
	// here we are using whatever the 'main' (only) window is.
	TOptional<FAndroidWindow::FNativeAccessor> MainWindowAccessor = FAndroidWindowManager::Get().GetMainWindowAsNativeAccessor(false);
	FAndroidAppEntry::ReInitWindow(MainWindowAccessor);
	FAndroidApplication::OnWindowSizeChanged();

	if (FAndroidMisc::UseNewWindowBehavior())
	{
		// Unblock rendering now that viewport has been resized to match the new surface
		const auto& OnWindowCreatedCallback = FPlatformDynamicRHI::GetRHIOnWindowCreatedCallback();
		if (OnWindowCreatedCallback)
		{
			OnWindowCreatedCallback();
		}
	}
}

void FAppEventManager::ExecWindowResized()
{
	if (bRunning)
	{
		FlushRenderingCommands();
	}
	FAndroidWindow::InvalidateCachedScreenRect();

	// ReInit with the new window handle
	// this code has no knowledge of multiple windows.
	// here we are using whatever the 'main' (only) window is...
	TOptional<FAndroidWindow::FNativeAccessor> MainWindowAccessor = FAndroidWindowManager::Get().GetMainWindowAsNativeAccessor(false);
	FAndroidAppEntry::ReInitWindow(MainWindowAccessor);
	FAndroidApplication::OnWindowSizeChanged();
}

void FAppEventManager::PauseAudio()
{
	if (!GEngine || !GEngine->IsInitialized())
	{
		UE_LOGF(LogAndroidEvents, Log, "Engine not initialized, not pausing Android audio");
		return;
	}

	bAudioPaused = true;
	UE_LOGF(LogAndroidEvents, Log, "Android pause audio");

	FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
	if (AudioDevice)
	{
		FAudioCommandFence Fence;
		Fence.BeginFence();
		Fence.Wait();

		AudioDevice->SuspendContext();
	}
}


void FAppEventManager::ResumeAudio()
{
	if (!GEngine || !GEngine->IsInitialized())
	{
		UE_LOGF(LogAndroidEvents, Log, "Engine not initialized, not resuming Android audio");
		return;
	}

	bAudioPaused = false;
	UE_LOGF(LogAndroidEvents, Log, "Android resume audio");

	FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
	if (AudioDevice)
	{
		AudioDevice->ResumeContext();
	}
}


void FAppEventManager::EnqueueAppEvent(EAppEventState InState, FAppEventData&& InData)
{
	FAppEventPacket Event;
	Event.State = InState;
	Event.Data = InData;

	int rc = pthread_mutex_lock(&QueueMutex);
	check(rc == 0);
	Queue.Enqueue(Event);

	if (EmptyQueueHandlerEvent)
	{
		EmptyQueueHandlerEvent->Reset();
	}

	rc = pthread_mutex_unlock(&QueueMutex);
	check(rc == 0);

	STANDALONE_DEBUG_LOGf(LogAndroidEvents, TEXT("LogAndroidEvents::EnqueueAppEvent : %u, [width=%d, height=%d], tid = %d, %s"), InState, InData.WindowWidth, InData.WindowHeight, gettid(), GetAppEventName(InState));
}

FAppEventPacket FAppEventManager::DequeueAppEvent()
{
	int rc = pthread_mutex_lock(&QueueMutex);
	check(rc == 0);

	FAppEventPacket OutData;
	Queue.Dequeue( OutData );

	rc = pthread_mutex_unlock(&QueueMutex);
	check(rc == 0);

	UE_LOGF(LogAndroidEvents, Display, "LogAndroidEvents::DequeueAppEvent : %u, [width=%d, height=%d], %ls", OutData.State, OutData.Data.WindowWidth, OutData.Data.WindowHeight, GetAppEventName(OutData.State))

	return OutData;
}


bool FAppEventManager::IsGamePaused()
{
	return !bRunning;
}


bool FAppEventManager::IsGameInFocus()
{
#if USE_ANDROID_STANDALONE
	return (bWindowInFocus && bHaveWindow && bHaveGame);
#else
	return (bWindowInFocus && bHaveWindow);
#endif
}


bool FAppEventManager::WaitForEventInQueue(EAppEventState InState, double TimeoutSeconds)
{
	bool FoundEvent = false;
	double StopTime = FPlatformTime::Seconds() + TimeoutSeconds;

	TQueue<FAppEventPacket, EQueueMode::Spsc> HoldingQueue;
	while (!FoundEvent)
	{
		int rc = pthread_mutex_lock(&QueueMutex);
		check(rc == 0);

		// Copy the existing queue (and check for our event)
		while (!Queue.IsEmpty())
		{
			FAppEventPacket OutData;
			Queue.Dequeue(OutData);

			if (OutData.State == InState)
				FoundEvent = true;

			HoldingQueue.Enqueue(OutData);
		}

		if (FoundEvent)
			break;

		// Time expired?
		if (FPlatformTime::Seconds() > StopTime)
			break;

		// Unlock for new events and wait a bit before trying again
		rc = pthread_mutex_unlock(&QueueMutex);
		check(rc == 0);
		FPlatformProcess::Sleep(0.01f);
	}

	// Add events back to queue from holding
	while (!HoldingQueue.IsEmpty())
	{
		FAppEventPacket OutData;
		HoldingQueue.Dequeue(OutData);
		Queue.Enqueue(OutData);
	}

	int rc = pthread_mutex_unlock(&QueueMutex);
	check(rc == 0);

	return FoundEvent;
}

#endif
