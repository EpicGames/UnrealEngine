// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidJNI.h"

#if USE_ANDROID_JNI
#include "Android/AndroidApplication.h"
#include "Android/AndroidInputInterface.h"
#include "Android/AndroidJniGameActivity.h"
#include "Android/AndroidJniNativeCalls.h"
#include "Android/AndroidPlatformCrashContext.h"
#include "Android/AndroidStats.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/PlatformStackWalk.h"
#include "Math/Vector.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/DateTime.h"
#include "Misc/EmbeddedCommunication.h"
#include "Misc/FeedbackContext.h"
#include "UnrealEngine.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"

THIRD_PARTY_INCLUDES_START
#include <android/asset_manager.h>
#include <android/asset_manager_jni.h>
THIRD_PARTY_INCLUDES_END

#define JNI_CURRENT_VERSION JNI_VERSION_1_6

extern JavaVM* GJavaVM;

// Pointer to target widget for virtual keyboard contents
static TWeakPtr<IVirtualKeyboardEntry> VirtualKeyboardWidget;

//virtualKeyboard shown
static volatile bool GVirtualKeyboardShown = false;
//WebView shown
static volatile bool GWebViewShown = false;

extern FString GFilePathBase;
extern FString GInternalFilePath;
extern FString GExternalFilePath;
extern FString GFontPathBase;
extern bool GOBBinAPK;
extern bool GOverrideAndroidLogDir;
extern FString GOBBFilePathBase;
extern FString GOBBMainFilePath;
extern FString GOBBPatchFilePath;
extern FString GOBBOverflow1FilePath;
extern FString GOBBOverflow2FilePath;
extern FString GAPKFilename;

// NOTE: We don't want WebRTC logic to always run. 
// Plugins that wish to use WebRTC should provide an implementation of this method.
extern "C" void InitWebRTCOnAndroid(JavaVM*) __attribute__((weak));

FOnActivityResult FJavaWrapper::OnActivityResultDelegate;
FOnSafetyNetAttestationResult FJavaWrapper::OnSafetyNetAttestationResultDelegate;
FOnRouteServiceIntent FJavaWrapper::OnRouteServiceIntentDelegate;

//////////////////////////////////////////////////////////////////////////

#if UE_BUILD_SHIPPING
// always clear any exceptions in Shipping
#define CHECK_JNI_RESULT(Id) if (Id == 0) { Env->ExceptionClear(); }
#else
#define CHECK_JNI_RESULT(Id) \
if (Id == 0) \
{ \
	if (!bIsOptional) { Env->ExceptionDescribe(); checkf(Id != 0, TEXT("Failed to find " #Id)); } \
	Env->ExceptionClear(); \
}
#endif

#define CHECK_JNI_METHOD(Id) checkf(Id != nullptr, TEXT("Failed to find " #Id));

jobject AndroidJavaEnv::GetGameActivityThis()
{
	return FJavaWrapper::GameActivityThis;
}

void FJavaWrapper::FindClassesAndMethods(JNIEnv* Env)
{
	auto bIsOptional = false;
	GameActivityClassID = UE::Jni::Class<UE::Jni::FGameActivity>;
	AndroidThunkJava_ShowConsoleWindow = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ShowConsoleWindow", "(Ljava/lang/String;)V", bIsOptional);
    AndroidThunkJava_ShowVirtualKeyboardInputDialog = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ShowVirtualKeyboardInputDialog", "(ILjava/lang/String;Ljava/lang/String;)V", bIsOptional);
    AndroidThunkJava_HideVirtualKeyboardInputDialog = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_HideVirtualKeyboardInputDialog", "()V", bIsOptional);
	AndroidThunkJava_ShowVirtualKeyboardInput = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ShowVirtualKeyboardInput", "(ILjava/lang/String;Ljava/lang/String;)V", bIsOptional);
	AndroidThunkJava_HideVirtualKeyboardInput = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_HideVirtualKeyboardInput", "()V", bIsOptional);
	AndroidThunkJava_LaunchURL = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_LaunchURL", "(Ljava/lang/String;)V", bIsOptional);
	AndroidThunkJava_GetAssetManager = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetAssetManager", "()Landroid/content/res/AssetManager;", bIsOptional);
	AndroidThunkJava_Minimize = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_Minimize", "()V", bIsOptional);
    AndroidThunkJava_ClipboardCopy = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ClipboardCopy", "(Ljava/lang/String;)V", bIsOptional);
    AndroidThunkJava_ClipboardPaste = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ClipboardPaste", "()Ljava/lang/String;", bIsOptional);
	AndroidThunkJava_ForceQuit = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ForceQuit", "()V", bIsOptional);
	AndroidThunkJava_GetFontDirectory = FindStaticMethod(Env, GameActivityClassID, "AndroidThunkJava_GetFontDirectory", "()Ljava/lang/String;", bIsOptional);
	AndroidThunkJava_Vibrate = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_Vibrate", "(II)V", bIsOptional);
	AndroidThunkJava_IsMusicActive = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_IsMusicActive", "()Z", bIsOptional);
	AndroidThunkJava_IsScreensaverEnabled = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_IsScreensaverEnabled", "()Z", bIsOptional);
	AndroidThunkJava_KeepScreenOn = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_KeepScreenOn", "(Z)V", bIsOptional);
	AndroidThunkJava_InitHMDs = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_InitHMDs", "()V", bIsOptional);
	AndroidThunkJava_DismissSplashScreen = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_DismissSplashScreen", "()V", bIsOptional);
	AndroidThunkJava_ShowProgressDialog = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ShowProgressDialog", "(ZLjava/lang/String;ZI)V", bIsOptional);
	AndroidThunkJava_UpdateProgressDialog = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_UpdateProgressDialog", "(I)V", bIsOptional);
	AndroidThunkJava_GetInputDeviceInfo = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetInputDeviceInfo", "(I)Lcom/epicgames/unreal/GameActivity$InputDeviceInfo;", bIsOptional);
	AndroidThunkJava_SetInputDeviceVibrators = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_SetInputDeviceVibrators", "(IIIII)Z", bIsOptional);
	AndroidThunkJava_IsGamepadAttached = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_IsGamepadAttached", "()Z", bIsOptional);
	AndroidThunkJava_HasMetaDataKey = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_HasMetaDataKey", "(Ljava/lang/String;)Z", bIsOptional);
	AndroidThunkJava_GetMetaDataBoolean = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetMetaDataBoolean", "(Ljava/lang/String;)Z", bIsOptional);
	AndroidThunkJava_GetMetaDataInt = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetMetaDataInt", "(Ljava/lang/String;)I", bIsOptional);
	AndroidThunkJava_GetMetaDataLong = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetMetaDataLong", "(Ljava/lang/String;)J", bIsOptional);
	AndroidThunkJava_GetMetaDataFloat = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetMetaDataFloat", "(Ljava/lang/String;)F", bIsOptional);
	AndroidThunkJava_GetMetaDataString = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetMetaDataString", "(Ljava/lang/String;)Ljava/lang/String;", bIsOptional);
	AndroidThunkJava_SetSustainedPerformanceMode = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_SetSustainedPerformanceMode", "(Z)V", bIsOptional);
	AndroidThunkJava_ShowHiddenAlertDialog = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ShowHiddenAlertDialog", "()V", bIsOptional);
	AndroidThunkJava_LocalNotificationScheduleAtTime = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_LocalNotificationScheduleAtTime", "(Ljava/lang/String;ZLjava/lang/String;Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;I)I", bIsOptional);
	AndroidThunkJava_LocalNotificationClearAll = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_LocalNotificationClearAll", "()V", bIsOptional);
	AndroidThunkJava_LocalNotificationExists = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_LocalNotificationExists", "(I)Z", bIsOptional);
	AndroidThunkJava_LocalNotificationGetLaunchNotification = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_LocalNotificationGetLaunchNotification", "()Lcom/epicgames/unreal/GameActivity$LaunchNotification;", bIsOptional);
	AndroidThunkJava_LocalNotificationDestroyIfExists = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_LocalNotificationDestroyIfExists", "(I)Z", bIsOptional);
	AndroidThunkJava_GetNetworkConnectionType = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetNetworkConnectionType", "()I", bIsOptional);
	AndroidThunkJava_AddNetworkListener = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_AddNetworkListener", "()V", bIsOptional);
	AndroidThunkJava_RemoveNetworkListener = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_AddNetworkListener", "()V", bIsOptional);
	AndroidThunkJava_GetAndroidId = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetAndroidId", "()Ljava/lang/String;", bIsOptional);
	AndroidThunkJava_ShareURL = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ShareURL", "(Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;II)V", bIsOptional);
	AndroidThunkJava_IsPackageInstalled = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_IsPackageInstalled", "(Ljava/lang/String;)Z", bIsOptional);
	AndroidThunkJava_LaunchPackage = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_LaunchPackage", "(Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;)Z", bIsOptional);
	AndroidThunkJava_SendBroadcast = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_SendBroadcast", "(Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;Z)Z", bIsOptional);
	AndroidThunkJava_HasIntentExtrasKey = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_HasIntentExtrasKey", "(Ljava/lang/String;)Z", bIsOptional);
	AndroidThunkJava_GetIntentExtrasBoolean = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetIntentExtrasBoolean", "(Ljava/lang/String;)Z", bIsOptional);
	AndroidThunkJava_GetIntentExtrasInt = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetIntentExtrasInt", "(Ljava/lang/String;)I", bIsOptional);
	AndroidThunkJava_GetIntentExtrasString = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetIntentExtrasString", "(Ljava/lang/String;)Ljava/lang/String;", bIsOptional);
	AndroidThunkJava_GetIntentDataAsString = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetIntentDataAsString", "()Ljava/lang/String;", bIsOptional);
	AndroidThunkJava_PushSensorEvents = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_PushSensorEvents", "()V", bIsOptional);
	AndroidThunkJava_SetOrientation = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_SetOrientation", "(I)V", bIsOptional);
	AndroidThunkJava_HasSharedPreference = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_HasSharedPreference", "(Ljava/lang/String;Ljava/lang/String;)Z", bIsOptional);
	AndroidThunkJava_SetSharedPreferenceBoolean = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_SetSharedPreferenceBoolean", "(Ljava/lang/String;Ljava/lang/String;Z)V", bIsOptional);
	AndroidThunkJava_GetSharedPreferenceBoolean = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetSharedPreferenceBoolean", "(Ljava/lang/String;Ljava/lang/String;Z)Z", bIsOptional);
	AndroidThunkJava_SetSharedPreferenceInt = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_SetSharedPreferenceInt", "(Ljava/lang/String;Ljava/lang/String;I)V", bIsOptional);
	AndroidThunkJava_GetSharedPreferenceInt = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetSharedPreferenceInt", "(Ljava/lang/String;Ljava/lang/String;I)I", bIsOptional);
	AndroidThunkJava_SetSharedPreferenceString = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_SetSharedPreferenceString", "(Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;)V", bIsOptional);
	AndroidThunkJava_GetSharedPreferenceString = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetSharedPreferenceString", "(Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;)Ljava/lang/String;", bIsOptional);
	AndroidThunkJava_DeleteSharedPreference = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_DeleteSharedPreference", "(Ljava/lang/String;Ljava/lang/String;)V", bIsOptional);
	AndroidThunkJava_DeleteSharedPreferenceGroup = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_DeleteSharedPreferenceGroup", "(Ljava/lang/String;)V", bIsOptional);
	AndroidThunkJava_GetCacheDir = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetCacheDir", "()Ljava/lang/String;", bIsOptional);

	// Screen capture/recording permission
	AndroidThunkJava_IsScreenCaptureDisabled = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_IsScreenCaptureDisabled", "()Z", bIsOptional);
	AndroidThunkJava_DisableScreenCapture = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_DisableScreenCapture", "(Z)V", bIsOptional);

	// Native view click filtering
	AndroidThunkJava_HasExtraIgnoreView = FindMethod(Env, GameActivityClassID, "HasExtraIgnoreView", "()Z", bIsOptional);;

	// this is optional - only inserted if OpenXR plugin enabled
	AndroidThunkJava_IsOpenXRImmersiveHMDApplication = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_IsOpenXRImmersiveHMDApplication", "()Z", true);

	// this is optional - only inserted if bPackageForMetaQuest
	AndroidThunkJava_IsOculusMobileApplication = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_IsOculusMobileApplication", "()Z", true);

	// this is optional - only inserted if GCM plugin enabled
	AndroidThunkJava_RegisterForRemoteNotifications = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_RegisterForRemoteNotifications", "()V", true);
	AndroidThunkJava_UnregisterForRemoteNotifications = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_UnregisterForRemoteNotifications", "()V", true);
	AndroidThunkJava_IsAllowedRemoteNotifications = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_IsAllowedRemoteNotifications", "()Z", true);

	// get field IDs for InputDeviceInfo class members
	InputDeviceInfoClass = FindClassGlobalRef(Env, "com/epicgames/unreal/GameActivity$InputDeviceInfo", bIsOptional);
	InputDeviceInfo_VendorId = FJavaWrapper::FindField(Env, InputDeviceInfoClass, "vendorId", "I", bIsOptional);
	InputDeviceInfo_ProductId = FJavaWrapper::FindField(Env, InputDeviceInfoClass, "productId", "I", bIsOptional);
	InputDeviceInfo_ControllerId = FJavaWrapper::FindField(Env, InputDeviceInfoClass, "controllerId", "I", bIsOptional);
	InputDeviceInfo_Name = FJavaWrapper::FindField(Env, InputDeviceInfoClass, "name", "Ljava/lang/String;", bIsOptional);
	InputDeviceInfo_Descriptor = FJavaWrapper::FindField(Env, InputDeviceInfoClass, "descriptor", "Ljava/lang/String;", bIsOptional);
	InputDeviceInfo_FeedbackMotorCount = FJavaWrapper::FindField(Env, InputDeviceInfoClass, "feedbackMotorCount", "I", bIsOptional);
	InputDeviceInfo_IsExternal = FJavaWrapper::FindField(Env, InputDeviceInfoClass, "isExternal", "Z", bIsOptional);

	/** GooglePlay services */
	FindGooglePlayMethods(Env);
	/** GooglePlay billing services */
	FindGooglePlayBillingMethods(Env);

	// get field IDs for LaunchNotificationClass class members
	LaunchNotificationClass = FindClassGlobalRef(Env, "com/epicgames/unreal/GameActivity$LaunchNotification", bIsOptional);
	LaunchNotificationUsed = FJavaWrapper::FindField(Env, LaunchNotificationClass, "used", "Z", bIsOptional);
	LaunchNotificationEvent = FJavaWrapper::FindField(Env, LaunchNotificationClass, "event", "Ljava/lang/String;", bIsOptional);
	LaunchNotificationFireDate = FJavaWrapper::FindField(Env, LaunchNotificationClass, "fireDate", "I", bIsOptional);

	ThreadClass = FindClassGlobalRef(Env, "java/lang/Thread", bIsOptional);
	CurrentThreadMethod = FindStaticMethod(Env, ThreadClass, "currentThread", "()Ljava/lang/Thread;", bIsOptional);
	SetNameMethod = FindMethod(Env, ThreadClass, "setName", "(Ljava/lang/String;)V", bIsOptional);

	AndroidThunkJava_RestartApplication = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_RestartApplication", "(Ljava/lang/String;)V", bIsOptional);

	// display refresh rates
	AndroidThunkJava_GetNativeDisplayRefreshRate = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetNativeDisplayRefreshRate", "()I", bIsOptional);
	AndroidThunkJava_SetNativeDisplayRefreshRate = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_SetNativeDisplayRefreshRate", "(I)Z", bIsOptional);
	AndroidThunkJava_GetSupportedNativeDisplayRefreshRates = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_GetSupportedNativeDisplayRefreshRates", "()[I", bIsOptional);

	// motion controls
	AndroidThunkJava_EnableMotion = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_EnableMotion", "(Z)V", bIsOptional);

	// audio focus
	AndroidThunkJava_OnAudioFocusChangedListenersUpdated = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_OnAudioFocusChangedListenersUpdated", "(Z)V", bIsOptional);

	// the rest are optional
	bIsOptional = true;

	// SurfaceView functionality for view scaling on some devices
	AndroidThunkJava_UseSurfaceViewWorkaround = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_UseSurfaceViewWorkaround", "()V", bIsOptional);
	AndroidThunkJava_SetDesiredViewSize = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_SetDesiredViewSize", "(II)V", bIsOptional);

	AndroidThunkJava_VirtualInputIgnoreClick = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_VirtualInputIgnoreClick", "(II)Z", bIsOptional);

	// Multicast lock handling
	AndroidThunkJava_AcquireWifiManagerMulticastLock = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_AcquireWifiManagerMulticastLock", "()Z", bIsOptional);
	AndroidThunkJava_ReleaseWifiManagerMulticastLock = FindMethod(Env, GameActivityClassID, "AndroidThunkJava_ReleaseWifiManagerMulticastLock", "()V", bIsOptional);

	SetupEmbeddedCommunication(Env);
}

void FJavaWrapper::FindGooglePlayMethods(JNIEnv* Env)
{
	bool bIsOptional = true;

	// @todo split GooglePlay
	//	GoogleServicesClassID = FindClass(Env, "com/epicgames/unreal/GoogleServices", bIsOptional);
	GoogleServicesClassID = GameActivityClassID;
	AndroidThunkJava_ShowAdBanner = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_ShowAdBanner", "(Ljava/lang/String;Z)V", bIsOptional);
	AndroidThunkJava_HideAdBanner = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_HideAdBanner", "()V", bIsOptional);
	AndroidThunkJava_CloseAdBanner = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_CloseAdBanner", "()V", bIsOptional);
	AndroidThunkJava_LoadInterstitialAd = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_LoadInterstitialAd", "(Ljava/lang/String;)V", bIsOptional);
	AndroidThunkJava_IsInterstitialAdAvailable = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IsInterstitialAdAvailable", "()Z", bIsOptional);
	AndroidThunkJava_IsInterstitialAdRequested = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IsInterstitialAdRequested", "()Z", bIsOptional);
	AndroidThunkJava_ShowInterstitialAd = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_ShowInterstitialAd", "()V", bIsOptional);
	AndroidThunkJava_GetAdvertisingId = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_GetAdvertisingId", "()Ljava/lang/String;", bIsOptional);
}
void FJavaWrapper::FindGooglePlayBillingMethods(JNIEnv* Env)
{
	// In app purchase functionality
	bool bSupportsInAppPurchasing = false;
	if (!GConfig->GetBool(TEXT("OnlineSubsystemGooglePlay.Store"), TEXT("bSupportsInAppPurchasing"), bSupportsInAppPurchasing, GEngineIni))
	{
		FPlatformMisc::LowLevelOutputDebugString(TEXT("[JNI] - Failed to determine if app purchasing is enabled!"));
	}
	bool bIsStoreOptional = !bSupportsInAppPurchasing;
	
	JavaStringClass = UE::Jni::Class<UE::Jni::Java::Lang::FString>;
	AndroidThunkJava_IapSetupService = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapSetupService", "(Ljava/lang/String;Z)V", bIsStoreOptional);
	AndroidThunkJava_IapQueryInAppPurchases = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapQueryInAppPurchases", "([Ljava/lang/String;)Z", bIsStoreOptional);
	AndroidThunkJava_IapBeginPurchase = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapBeginPurchase", "([Ljava/lang/String;Ljava/lang/String;)Z", bIsStoreOptional);
	AndroidThunkJava_IapIsAllowedToMakePurchases = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapIsAllowedToMakePurchases", "()Z", bIsStoreOptional);
	AndroidThunkJava_IapAcknowledgePurchase = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapAcknowledgePurchase", "(Ljava/lang/String;)Z", bIsStoreOptional);
	AndroidThunkJava_IapConsumePurchase = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapConsumePurchase", "(Ljava/lang/String;)Z", bIsStoreOptional);
	AndroidThunkJava_IapQueryExistingPurchases = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapQueryExistingPurchases", "()Z", bIsStoreOptional);
	AndroidThunkJava_IapGetBillingCountryCode = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapGetBillingCountryCode", "()Ljava/lang/String;", bIsStoreOptional);
	AndroidThunkJava_IapGetIsAlternativeBillingOnlyAvailable = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapGetIsAlternativeBillingOnlyAvailable", "()Z", bIsStoreOptional);
	AndroidThunkJava_IapShowAlternativeBillingOnlyInformationDialog = FindMethod(Env, GoogleServicesClassID, "AndroidThunkJava_IapShowAlternativeBillingOnlyInformationDialog", "()Z", bIsStoreOptional);
}

jclass FJavaWrapper::FindClass(JNIEnv* Env, const ANSICHAR* ClassName, bool bIsOptional)
{
	jclass Class = Env->FindClass(ClassName);
	CHECK_JNI_RESULT(Class);
	return Class;
}

jclass FJavaWrapper::FindClassGlobalRef(JNIEnv* Env, const ANSICHAR* ClassName, bool bIsOptional)
{
	auto LocalClass = NewScopedJavaObject(Env, Env->FindClass(ClassName));
	CHECK_JNI_RESULT(*LocalClass);
	if (LocalClass)
	{
		return (jclass)Env->NewGlobalRef(*LocalClass);
	}
	return nullptr;
}

jmethodID FJavaWrapper::FindMethod(JNIEnv* Env, jclass Class, const ANSICHAR* MethodName, const ANSICHAR* MethodSignature, bool bIsOptional)
{
	jmethodID Method = Class == NULL ? NULL : Env->GetMethodID(Class, MethodName, MethodSignature);
	CHECK_JNI_RESULT(Method);
	return Method;
}

jmethodID FJavaWrapper::FindStaticMethod(JNIEnv* Env, jclass Class, const ANSICHAR* MethodName, const ANSICHAR* MethodSignature, bool bIsOptional)
{
	jmethodID Method = Class == NULL ? NULL : Env->GetStaticMethodID(Class, MethodName, MethodSignature);
	CHECK_JNI_RESULT(Method);
	return Method;
}

jfieldID FJavaWrapper::FindField(JNIEnv* Env, jclass Class, const ANSICHAR* FieldName, const ANSICHAR* FieldType, bool bIsOptional)
{
	jfieldID Field = Class == NULL ? NULL : Env->GetFieldID(Class, FieldName, FieldType);
	CHECK_JNI_RESULT(Field);
	return Field;
}

void FJavaWrapper::CallVoidMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...)
{
	// make sure the function exists
	if (Method == NULL || Object == NULL)
	{
		return;
	}

	va_list Args;
	va_start(Args, Method);
	Env->CallVoidMethodV(Object, Method, Args);
	va_end(Args);
}

jobject FJavaWrapper::CallObjectMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...)
{
	if (Method == NULL || Object == NULL)
	{
		return NULL;
	}

	va_list Args;
	va_start(Args, Method);
	jobject Return = Env->CallObjectMethodV(Object, Method, Args);
	va_end(Args);

	return Return;
}

int32 FJavaWrapper::CallIntMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...)
{
	if (Method == NULL || Object == NULL)
	{
		return 0;
	}

	va_list Args;
	va_start(Args, Method);
	jint Return = Env->CallIntMethodV(Object, Method, Args);
	va_end(Args);

	return (int32)Return;
}

int64 FJavaWrapper::CallLongMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...)
{
	if (Method == NULL || Object == NULL)
	{
		return 0;
	}

	va_list Args;
	va_start(Args, Method);
	jlong Return = Env->CallLongMethodV(Object, Method, Args);
	va_end(Args);

	return (int64)Return;
}

float FJavaWrapper::CallFloatMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...)
{
	if (Method == NULL || Object == NULL)
	{
		return 0.0f;
	}

	va_list Args;
	va_start(Args, Method);
	jfloat Return = Env->CallFloatMethodV(Object, Method, Args);
	va_end(Args);

	return (float)Return;
}

double FJavaWrapper::CallDoubleMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...)
{
	if (Method == NULL || Object == NULL)
	{
		return 0.0;
	}

	va_list Args;
	va_start(Args, Method);
	jdouble Return = Env->CallDoubleMethodV(Object, Method, Args);
	va_end(Args);

	return (double)Return;
}

bool FJavaWrapper::CallBooleanMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...)
{
	if (Method == NULL || Object == NULL)
	{
		return false;
	}

	va_list Args;
	va_start(Args, Method);
	jboolean Return = Env->CallBooleanMethodV(Object, Method, Args);
	va_end(Args);

	return (bool)Return;
}

void FJavaWrapper::CallStaticVoidMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...)
{
	if (Method == NULL || Clazz == NULL)
	{
		return;
	}

	va_list Args;
	va_start(Args, Method);
	Env->CallStaticVoidMethodV(Clazz, Method, Args);
	va_end(Args);
}

jobject FJavaWrapper::CallStaticObjectMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...)
{
	if (Method == NULL || Clazz == NULL)
	{
		return nullptr;
	}

	va_list Args;
	va_start(Args, Method);
	jobject Return = Env->CallStaticObjectMethodV(Clazz, Method, Args);
	va_end(Args);

	return Return;
}

int32 FJavaWrapper::CallStaticIntMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...)
{
	if (Method == NULL || Clazz == NULL)
	{
		return 0;
	}

	va_list Args;
	va_start(Args, Method);
	jint Return = Env->CallStaticIntMethod(Clazz, Method, Args);
	va_end(Args);

	return (int32)Return;
}

int64 FJavaWrapper::CallStaticLongMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...)
{
	if (Method == NULL || Clazz == NULL)
	{
		return 0;
	}

	va_list Args;
	va_start(Args, Method);
	jlong Return = Env->CallStaticLongMethod(Clazz, Method, Args);
	va_end(Args);

	return (int64)Return;
}

float FJavaWrapper::CallStaticFloatMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...)
{
	if (Method == NULL || Clazz == NULL)
	{
		return 0.f;
	}

	va_list Args;
	va_start(Args, Method);
	jfloat Return = Env->CallStaticFloatMethod(Clazz, Method, Args);
	va_end(Args);

	return (float)Return;
}

double FJavaWrapper::CallStaticDoubleMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...)
{
	if (Method == NULL || Clazz == NULL)
	{
		return 0.;
	}

	va_list Args;
	va_start(Args, Method);
	jdouble Return = Env->CallStaticDoubleMethod(Clazz, Method, Args);
	va_end(Args);

	return (double)Return;
}

bool FJavaWrapper::CallStaticBooleanMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...)
{
	if (Method == NULL || Clazz == NULL)
	{
		return false;
	}

	va_list Args;
	va_start(Args, Method);
	jboolean Return = Env->CallStaticBooleanMethod(Clazz, Method, Args);
	va_end(Args);

	return (bool)Return;
}

//Declare all the static members of the class defs 
jclass FJavaWrapper::GameActivityClassID;
jobject FJavaWrapper::GameActivityThis;
jmethodID FJavaWrapper::AndroidThunkJava_ShowConsoleWindow;
jmethodID FJavaWrapper::AndroidThunkJava_ShowVirtualKeyboardInputDialog;
jmethodID FJavaWrapper::AndroidThunkJava_HideVirtualKeyboardInputDialog;
jmethodID FJavaWrapper::AndroidThunkJava_ShowVirtualKeyboardInput;
jmethodID FJavaWrapper::AndroidThunkJava_HideVirtualKeyboardInput;
jmethodID FJavaWrapper::AndroidThunkJava_LaunchURL;
jmethodID FJavaWrapper::AndroidThunkJava_GetAssetManager;
jmethodID FJavaWrapper::AndroidThunkJava_Minimize;
jmethodID FJavaWrapper::AndroidThunkJava_ClipboardCopy;
jmethodID FJavaWrapper::AndroidThunkJava_ClipboardPaste;
jmethodID FJavaWrapper::AndroidThunkJava_ForceQuit;
jmethodID FJavaWrapper::AndroidThunkJava_GetFontDirectory;
jmethodID FJavaWrapper::AndroidThunkJava_Vibrate;
jmethodID FJavaWrapper::AndroidThunkJava_IsMusicActive;
jmethodID FJavaWrapper::AndroidThunkJava_IsScreensaverEnabled;
jmethodID FJavaWrapper::AndroidThunkJava_KeepScreenOn;
jmethodID FJavaWrapper::AndroidThunkJava_InitHMDs;
jmethodID FJavaWrapper::AndroidThunkJava_DismissSplashScreen;
jmethodID FJavaWrapper::AndroidThunkJava_ShowProgressDialog;
jmethodID FJavaWrapper::AndroidThunkJava_UpdateProgressDialog;
jmethodID FJavaWrapper::AndroidThunkJava_GetInputDeviceInfo;
jmethodID FJavaWrapper::AndroidThunkJava_SetInputDeviceVibrators;
jmethodID FJavaWrapper::AndroidThunkJava_IsGamepadAttached;
jmethodID FJavaWrapper::AndroidThunkJava_HasMetaDataKey;
jmethodID FJavaWrapper::AndroidThunkJava_GetMetaDataBoolean;
jmethodID FJavaWrapper::AndroidThunkJava_GetMetaDataInt;
jmethodID FJavaWrapper::AndroidThunkJava_GetMetaDataLong;
jmethodID FJavaWrapper::AndroidThunkJava_GetMetaDataFloat;
jmethodID FJavaWrapper::AndroidThunkJava_GetMetaDataString;
jmethodID FJavaWrapper::AndroidThunkJava_IsOpenXRImmersiveHMDApplication;
jmethodID FJavaWrapper::AndroidThunkJava_IsOculusMobileApplication;
jmethodID FJavaWrapper::AndroidThunkJava_RegisterForRemoteNotifications;
jmethodID FJavaWrapper::AndroidThunkJava_UnregisterForRemoteNotifications;
jmethodID FJavaWrapper::AndroidThunkJava_IsAllowedRemoteNotifications;
jmethodID FJavaWrapper::AndroidThunkJava_ShowHiddenAlertDialog;
jmethodID FJavaWrapper::AndroidThunkJava_LocalNotificationScheduleAtTime;
jmethodID FJavaWrapper::AndroidThunkJava_LocalNotificationClearAll;
jmethodID FJavaWrapper::AndroidThunkJava_LocalNotificationExists;
jmethodID FJavaWrapper::AndroidThunkJava_LocalNotificationGetLaunchNotification;
jmethodID FJavaWrapper::AndroidThunkJava_LocalNotificationDestroyIfExists;
jmethodID FJavaWrapper::AndroidThunkJava_GetNetworkConnectionType;
jmethodID FJavaWrapper::AndroidThunkJava_GetAndroidId;
jmethodID FJavaWrapper::AndroidThunkJava_ShareURL;
jmethodID FJavaWrapper::AndroidThunkJava_IsPackageInstalled;
jmethodID FJavaWrapper::AndroidThunkJava_LaunchPackage;
jmethodID FJavaWrapper::AndroidThunkJava_SendBroadcast;
jmethodID FJavaWrapper::AndroidThunkJava_HasIntentExtrasKey;
jmethodID FJavaWrapper::AndroidThunkJava_GetIntentExtrasBoolean;
jmethodID FJavaWrapper::AndroidThunkJava_GetIntentExtrasInt;
jmethodID FJavaWrapper::AndroidThunkJava_GetIntentExtrasString;
jmethodID FJavaWrapper::AndroidThunkJava_GetIntentDataAsString;
jmethodID FJavaWrapper::AndroidThunkJava_SetSustainedPerformanceMode;
jmethodID FJavaWrapper::AndroidThunkJava_PushSensorEvents;
jmethodID FJavaWrapper::AndroidThunkJava_IsScreenCaptureDisabled;
jmethodID FJavaWrapper::AndroidThunkJava_DisableScreenCapture;
jmethodID FJavaWrapper::AndroidThunkJava_HasExtraIgnoreView;
jmethodID FJavaWrapper::AndroidThunkJava_SetOrientation;
jmethodID FJavaWrapper::AndroidThunkJava_HasSharedPreference;
jmethodID FJavaWrapper::AndroidThunkJava_SetSharedPreferenceBoolean;
jmethodID FJavaWrapper::AndroidThunkJava_GetSharedPreferenceBoolean;
jmethodID FJavaWrapper::AndroidThunkJava_SetSharedPreferenceInt;
jmethodID FJavaWrapper::AndroidThunkJava_GetSharedPreferenceInt;
jmethodID FJavaWrapper::AndroidThunkJava_SetSharedPreferenceString;
jmethodID FJavaWrapper::AndroidThunkJava_GetSharedPreferenceString;
jmethodID FJavaWrapper::AndroidThunkJava_DeleteSharedPreference;
jmethodID FJavaWrapper::AndroidThunkJava_DeleteSharedPreferenceGroup;
jmethodID FJavaWrapper::AndroidThunkJava_GetCacheDir;

jclass FJavaWrapper::InputDeviceInfoClass;
jfieldID FJavaWrapper::InputDeviceInfo_VendorId;
jfieldID FJavaWrapper::InputDeviceInfo_ProductId;
jfieldID FJavaWrapper::InputDeviceInfo_ControllerId;
jfieldID FJavaWrapper::InputDeviceInfo_Name;
jfieldID FJavaWrapper::InputDeviceInfo_Descriptor;
jfieldID FJavaWrapper::InputDeviceInfo_FeedbackMotorCount;
jfieldID FJavaWrapper::InputDeviceInfo_IsExternal;

jclass FJavaWrapper::GoogleServicesClassID;
jobject FJavaWrapper::GoogleServicesThis;
jmethodID FJavaWrapper::AndroidThunkJava_ShowAdBanner;
jmethodID FJavaWrapper::AndroidThunkJava_HideAdBanner;
jmethodID FJavaWrapper::AndroidThunkJava_CloseAdBanner;
jmethodID FJavaWrapper::AndroidThunkJava_LoadInterstitialAd;
jmethodID FJavaWrapper::AndroidThunkJava_IsInterstitialAdAvailable;
jmethodID FJavaWrapper::AndroidThunkJava_IsInterstitialAdRequested;
jmethodID FJavaWrapper::AndroidThunkJava_ShowInterstitialAd;
jmethodID FJavaWrapper::AndroidThunkJava_GetAdvertisingId;

jclass FJavaWrapper::JavaStringClass;
jmethodID FJavaWrapper::AndroidThunkJava_IapSetupService;
jmethodID FJavaWrapper::AndroidThunkJava_IapQueryInAppPurchases;
jmethodID FJavaWrapper::AndroidThunkJava_IapBeginPurchase;
jmethodID FJavaWrapper::AndroidThunkJava_IapIsAllowedToMakePurchases;
jmethodID FJavaWrapper::AndroidThunkJava_IapQueryExistingPurchases;
jmethodID FJavaWrapper::AndroidThunkJava_IapAcknowledgePurchase;
jmethodID FJavaWrapper::AndroidThunkJava_IapConsumePurchase;
jmethodID FJavaWrapper::AndroidThunkJava_IapGetBillingCountryCode;
jmethodID FJavaWrapper::AndroidThunkJava_IapGetIsAlternativeBillingOnlyAvailable;
jmethodID FJavaWrapper::AndroidThunkJava_IapShowAlternativeBillingOnlyInformationDialog;

jmethodID FJavaWrapper::AndroidThunkJava_UseSurfaceViewWorkaround;
jmethodID FJavaWrapper::AndroidThunkJava_SetDesiredViewSize;

jmethodID FJavaWrapper::AndroidThunkJava_VirtualInputIgnoreClick;

jmethodID FJavaWrapper::AndroidThunkJava_RestartApplication;

jmethodID FJavaWrapper::AndroidThunkJava_GetSupportedNativeDisplayRefreshRates;
jmethodID FJavaWrapper::AndroidThunkJava_GetNativeDisplayRefreshRate;
jmethodID FJavaWrapper::AndroidThunkJava_SetNativeDisplayRefreshRate;

jmethodID FJavaWrapper::AndroidThunkJava_AddNetworkListener;
jmethodID FJavaWrapper::AndroidThunkJava_RemoveNetworkListener;

jmethodID FJavaWrapper::AndroidThunkJava_EnableMotion;

jclass FJavaWrapper::LaunchNotificationClass;
jfieldID FJavaWrapper::LaunchNotificationUsed;
jfieldID FJavaWrapper::LaunchNotificationEvent;
jfieldID FJavaWrapper::LaunchNotificationFireDate;

jclass FJavaWrapper::ThreadClass;
jmethodID FJavaWrapper::CurrentThreadMethod;
jmethodID FJavaWrapper::SetNameMethod;

jmethodID FJavaWrapper::AndroidThunkJava_AcquireWifiManagerMulticastLock;
jmethodID FJavaWrapper::AndroidThunkJava_ReleaseWifiManagerMulticastLock;

jmethodID FJavaWrapper::AndroidThunkJava_OnAudioFocusChangedListenersUpdated;

//Game-specific crash reporter
void EngineCrashHandler(const FGenericCrashContext& GenericContext)
{
	const FAndroidCrashContext& Context = static_cast<const FAndroidCrashContext&>(GenericContext);

	static int32 bHasEntered = 0;
	if (FPlatformAtomics::InterlockedCompareExchange(&bHasEntered, 1, 0) == 0)
	{
		const SIZE_T StackTraceSize = 65535;
		ANSICHAR StackTrace[StackTraceSize];
		StackTrace[0] = 0;

		// Walk the stack and dump it to the allocated memory.
		FPlatformStackWalk::StackWalkAndDump(StackTrace, StackTraceSize, 0, Context.Context);
		UE_LOGF(LogEngine, Error, "\n%ls\n", ANSI_TO_TCHAR(StackTrace));

		if (GLog)
		{
			GLog->Panic();
		}
		
		if (GWarn)
		{
			GWarn->Flush();
		}
	}
}

bool AndroidThunkCpp_IsScreensaverEnabled()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// call the java side
		return FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IsScreensaverEnabled);
	}
	return true;
}

void AndroidThunkCpp_KeepScreenOn(bool Enable)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// call the java side
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_KeepScreenOn, Enable);
	}
}

void AndroidThunkCpp_Vibrate(int32 Intensity, int32 Duration)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// call the java side
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_Vibrate, Intensity, Duration);
	}
}

// Call the Java side code for initializing VR HMD modules
void AndroidThunkCpp_InitHMDs()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_InitHMDs);
	}
}

void AndroidThunkCpp_DismissSplashScreen()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_DismissSplashScreen);
	}
}

void AndroidThunkCpp_ShowProgressDialog(bool bShow, const FString& Message, bool bHorizontal, int32 MaxValue)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto JavaMessage = FJavaHelper::ToJavaString(Env, Message);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ShowProgressDialog, bShow, *JavaMessage, bHorizontal, MaxValue);
	}
}

void AndroidThunkCpp_UpdateProgressDialog(int32 Value)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_UpdateProgressDialog, Value);
	}
}

bool AndroidThunkCpp_GetInputDeviceInfo(int32 deviceId, FAndroidInputDeviceInfo &results)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto deviceInfo = NewScopedJavaObject(Env, (jobject)Env->CallObjectMethod(FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetInputDeviceInfo, deviceId));
		if (Env->ExceptionCheck())
		{
			Env->ExceptionDescribe();
			Env->ExceptionClear();
		}
		else if (deviceInfo)
		{
			bool bIsOptional = false;
			results.DeviceId = deviceId;
			results.VendorId = (int32)Env->GetIntField(*deviceInfo, FJavaWrapper::InputDeviceInfo_VendorId);
			results.ProductId = (int32)Env->GetIntField(*deviceInfo, FJavaWrapper::InputDeviceInfo_ProductId);
			results.ControllerId = (int32)Env->GetIntField(*deviceInfo, FJavaWrapper::InputDeviceInfo_ControllerId);

			FScopedJavaObject JavaInputDeviceName = NewScopedJavaObject(Env, (jstring)Env->GetObjectField(*deviceInfo, FJavaWrapper::InputDeviceInfo_Name));
			const auto chars = Env->GetStringUTFChars(*JavaInputDeviceName, 0);
			results.Name = chars; // UTF8_TO_TCHAR(chars)
			Env->ReleaseStringUTFChars(*JavaInputDeviceName, chars);

			results.Descriptor = FJavaHelper::FStringFromLocalRef(Env, (jstring)Env->GetObjectField(*deviceInfo, FJavaWrapper::InputDeviceInfo_Descriptor));

			results.FeedbackMotorCount = (int32)Env->GetIntField(*deviceInfo, FJavaWrapper::InputDeviceInfo_FeedbackMotorCount);
			results.IsExternal = (bool)Env->GetBooleanField(*deviceInfo, FJavaWrapper::InputDeviceInfo_IsExternal);
			return true;
		}
	}

	// failed
	results.DeviceId = deviceId;
	results.VendorId = 0;
	results.ProductId = 0;
	results.ControllerId = -1;
	results.Name = TEXT("Unknown");
	results.Descriptor = FString("Unknown");
	results.FeedbackMotorCount = 0;
	return false;
}

bool AndroidThunkCpp_SetInputDeviceVibrators(int32 deviceId, int32 leftIntensity, int32 leftDuration, int32 rightIntensity, int32 rightDuration)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		return FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_SetInputDeviceVibrators, deviceId, leftIntensity, leftDuration, rightIntensity, rightDuration);
	}
	return false;
}

bool AndroidThunkCpp_VirtualInputIgnoreClick(int32 x, int32 y)
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_VirtualInputIgnoreClick, x, y);
	}
	return Result;
}

bool AndroidThunkCpp_HasExtraIgnoreView()
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_HasExtraIgnoreView);
	}
	return Result;
}

void AndroidThunkCpp_RestartApplication(const FString& IntentString)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, IntentString);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_RestartApplication, *Argument);
	}
}

UE::Jni::FGameActivity* UE::Jni::FGameActivity::Get()
{
	return reinterpret_cast<FGameActivity*>(FJavaWrapper::GameActivityThis);
}

void JNICALL UE::Jni::FGameActivity::nativeHandleNewIntentData(JNIEnv* jenv, jobject thiz, jstring JavaIntent)
{
	FAndroidMisc::HandleNewIntentUri(FJavaHelper::FStringFromParam(jenv, JavaIntent));
}

void JNICALL UE::Jni::FNativeCalls::WebViewVisible(JNIEnv* jenv, jclass clazz, jboolean bShown)
{
	GWebViewShown = bShown;
}

bool AndroidThunkCpp_IsWebViewShown()
{
	return GWebViewShown;
}

//Set GVirtualKeyboardShown.This function is declared in the Java-defined class, GameActivity.java: "public native void nativeVirtualKeyboardVisible(boolean bShown)"
void JNICALL UE::Jni::FGameActivity::nativeVirtualKeyboardVisible(JNIEnv* jenv, jobject thiz, jboolean bShown)
{
	GVirtualKeyboardShown = bShown;

	//remove reference so the object can be clicked again to show the virtual keyboard
	if (!bShown)
	{
		if (FTaskGraphInterface::IsRunning())
		{
			FGraphEventRef VirtualKeyboardShown = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
			{
				FAndroidApplication::Get()->OnVirtualKeyboardHidden().Broadcast();
        	}, TStatId(), NULL, ENamedThreads::GameThread);
		}
	}
}

bool AndroidThunkCpp_IsVirtualKeyboardShown()
{
	return GVirtualKeyboardShown;
}

bool AndroidThunkCpp_IsGamepadAttached()
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IsGamepadAttached);
	}
	return Result;
}

bool AndroidThunkCpp_HasMetaDataKey(const FString& Key)
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_HasMetaDataKey, *Argument);
	}
	return Result;
}

bool AndroidThunkCpp_GetMetaDataBoolean(const FString& Key)
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetMetaDataBoolean, *Argument);
	}
	return Result;
}

int32 AndroidThunkCpp_GetMetaDataInt(const FString& Key)
{
	int32 Result = 0;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetMetaDataInt, *Argument);
	}
	return Result;
}

int64 AndroidThunkCpp_GetMetaDataLong(const FString& Key)
{
	int64 Result = 0;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = (int64)FJavaWrapper::CallLongMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetMetaDataLong, *Argument);
	}
	return Result;
}

float AndroidThunkCpp_GetMetaDataFloat(const FString& Key)
{
	float Result = 0.0f;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = (float)FJavaWrapper::CallFloatMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetMetaDataFloat, *Argument);
	}
	return Result;
}

FString AndroidThunkCpp_GetMetaDataString(const FString& Key)
{
	FString Result = FString();
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetMetaDataString, *Argument));
	}
	return Result;
}

bool AndroidThunkCpp_HasIntentExtrasKey(const FString& Key)
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_HasIntentExtrasKey, *Argument);
	}
	return Result;
}

bool AndroidThunkCpp_GetIntentExtrasBoolean(const FString& Key)
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetIntentExtrasBoolean, *Argument);
	}
	return Result;
}

int32 AndroidThunkCpp_GetIntentExtrasInt(const FString& Key)
{
	int32 Result = 0;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetIntentExtrasInt, *Argument);
	}
	return Result;
}

FString AndroidThunkCpp_GetIntentExtrasString(const FString& Key)
{
	FString Result = FString();
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetIntentExtrasString, *Argument));
	}
	return Result;
}

FString AndroidThunkCpp_GetIntentDataAsString()
{
	FString Result = FString();
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetIntentDataAsString));
	}
	return Result;
}

void AndroidThunkCpp_SetSustainedPerformanceMode(bool bEnable)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_SetSustainedPerformanceMode, bEnable);
	}
}

void AndroidThunkCpp_PushSensorEvents()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_PushSensorEvents);
	}
}

bool AndroidThunkCpp_IsScreenCaptureDisabled()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// call the java side
		return FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IsScreenCaptureDisabled);
	}
	return true;
}

void AndroidThunkCpp_DisableScreenCapture(bool bDisable)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// call the java side
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_DisableScreenCapture, bDisable);
	}
}

void AndroidThunkCpp_ShowHiddenAlertDialog()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ShowHiddenAlertDialog);
	}
}

// call out to JNI to see if the application was packaged for OpenXR Immersive
bool AndroidThunkCpp_IsOpenXRImmersiveHMDApplication()
{
	static int32 IsOpenXRImmersiveHMDApplication = -1;

	if (IsOpenXRImmersiveHMDApplication == -1)
	{
		IsOpenXRImmersiveHMDApplication = 0;
		if (FJavaWrapper::AndroidThunkJava_IsOpenXRImmersiveHMDApplication)
		{
			if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
			{
				IsOpenXRImmersiveHMDApplication = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IsOpenXRImmersiveHMDApplication) ? 1 : 0;
			}
		}
	}
	return IsOpenXRImmersiveHMDApplication == 1;
}

bool AndroidThunkCpp_IsStandaloneApplication()
{
	return USE_ANDROID_STANDALONE != 0;
}

// call out to JNI to see if the application was packaged for Oculus Mobile
bool AndroidThunkCpp_IsOculusMobileApplication()
{
	static int32 IsOculusMobileApplication = -1;

	if (IsOculusMobileApplication == -1)
	{
		IsOculusMobileApplication = 0;
		if (FJavaWrapper::AndroidThunkJava_IsOculusMobileApplication)
		{
			if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
			{
				IsOculusMobileApplication = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IsOculusMobileApplication) ? 1 : 0;
			}
		}
	}
	return IsOculusMobileApplication == 1;
}

bool AndroidThunkCpp_IsXRImmersiveHMDApplication()
{
	return AndroidThunkCpp_IsOpenXRImmersiveHMDApplication() || AndroidThunkCpp_IsOculusMobileApplication();
}

// call optional remote notification registration
void AndroidThunkCpp_RegisterForRemoteNotifications()
{
	if (FJavaWrapper::AndroidThunkJava_RegisterForRemoteNotifications)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_RegisterForRemoteNotifications);
		}
	}
}

// call optional remote notification unregistration
void AndroidThunkCpp_UnregisterForRemoteNotifications()
{
	if (FJavaWrapper::AndroidThunkJava_UnregisterForRemoteNotifications)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_UnregisterForRemoteNotifications);
		}
	}
}

bool AndroidThunkCpp_IsAllowedRemoteNotifications()
{
	bool Result = false;
	if (FJavaWrapper::AndroidThunkJava_UnregisterForRemoteNotifications)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IsAllowedRemoteNotifications);
		}
	}

	return Result;
}

void AndroidThunkCpp_ShowConsoleWindow()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// figure out all the possible texture formats that are allowed
		TArray<FString> PossibleTargetPlatforms;
		FPlatformMisc::GetValidTargetPlatforms(PossibleTargetPlatforms);

		// separate the format suffixes with commas
		FString ConsoleText;
		for (int32 FormatIndex = 0; FormatIndex < PossibleTargetPlatforms.Num(); FormatIndex++)
		{
			const FString& Format = PossibleTargetPlatforms[FormatIndex];
			int32 UnderscoreIndex;
			if (Format.FindLastChar('_', UnderscoreIndex))
			{
				if (ConsoleText != TEXT(""))
				{
					ConsoleText += ", ";
				}

				ConsoleText += Format.Mid(UnderscoreIndex + 1);
			}
		}

		// call the java side
		auto ConsoleTextJava = FJavaHelper::ToJavaString(Env, ConsoleText);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ShowConsoleWindow, *ConsoleTextJava);
	}
}

void AndroidThunkCpp_ShowVirtualKeyboardInputDialog(TSharedPtr<IVirtualKeyboardEntry> TextWidget, int32 InputType, const FString& Label, const FString& Contents)
{
    if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
    {
        // remember target widget for contents
		VirtualKeyboardWidget = TextWidget;

		// call the java side
		auto LabelJava = FJavaHelper::ToJavaString(Env, Label);
		auto ContentsJava = FJavaHelper::ToJavaString(Env, Contents);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ShowVirtualKeyboardInputDialog, InputType, *LabelJava, *ContentsJava);
    }
}

void AndroidThunkCpp_HideVirtualKeyboardInputDialog()
{
    // Make sure virtual keyboard currently open
	if (!VirtualKeyboardWidget.IsValid())
	{
		return;
	}
    
    if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
    {
        // ignore anything it might return
		VirtualKeyboardWidget.Reset();

        // call the java side
        FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_HideVirtualKeyboardInputDialog);
	}
}

// This is called from the ViewTreeObserver.OnGlobalLayoutListener in GameActivity
void JNICALL UE::Jni::FGameActivity::nativeVirtualKeyboardShown(JNIEnv* jenv, jobject thiz, jint left, jint top, jint right, jint bottom)
{
	FPlatformRect ScreenRect(left, top, right, bottom);

	if (FTaskGraphInterface::IsRunning())
	{
		FGraphEventRef VirtualKeyboardShown = FFunctionGraphTask::CreateAndDispatchWhenReady([ScreenRect]()
		{
			FAndroidApplication::Get()->OnVirtualKeyboardShown().Broadcast(ScreenRect);
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}

void AndroidThunkCpp_HideVirtualKeyboardInput()
{
	// Make sure virtual keyboard currently open
	if (!VirtualKeyboardWidget.IsValid())
	{
		return;
	}

	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// call the java side
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_HideVirtualKeyboardInput);
	}
}

void AndroidThunkCpp_ShowVirtualKeyboardInput(TSharedPtr<IVirtualKeyboardEntry> TextWidget, int32 InputType, const FString& Label, const FString& Contents)
{
	//#jira UE-49139 Tapping in the same text box doesn't make the virtual keyboard disappear
	if (VirtualKeyboardWidget.HasSameObject(TextWidget.Get()))
	{
		FPlatformMisc::LowLevelOutputDebugString(TEXT("[JNI] - AndroidThunkCpp_ShowVirtualKeyboardInput same control"));
		AndroidThunkCpp_HideVirtualKeyboardInput();
	}
	else if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// remember target widget for contents
		VirtualKeyboardWidget = TextWidget;

		// call the java side
		auto LabelJava = FJavaHelper::ToJavaString(Env, Label);
		auto ContentsJava = FJavaHelper::ToJavaString(Env, Contents);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ShowVirtualKeyboardInput, InputType, *LabelJava, *ContentsJava);
	}
}

static void nativeVirtualKeyboardResult(JNIEnv* jenv, jobject thiz, jboolean update, jstring contents, ETextEntryType textEntryType)
{
	// update text widget with new contents if OK pressed
	if (update == JNI_TRUE)
	{
		if (VirtualKeyboardWidget.IsValid())
		{
			FString Contents = FJavaHelper::FStringFromParam(jenv, contents);

			// call to set the widget text on game thread
			if (FTaskGraphInterface::IsRunning())
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady([Contents=Contents, textEntryType=textEntryType]()
				{
					TSharedPtr<IVirtualKeyboardEntry> LockedKeyboardWidget(VirtualKeyboardWidget.Pin());
					if (LockedKeyboardWidget.IsValid())
					{
						LockedKeyboardWidget->SetTextFromVirtualKeyboard(FText::FromString(Contents), textEntryType);
					}

					// release reference
					VirtualKeyboardWidget.Reset();
				}, TStatId(), NULL, ENamedThreads::GameThread);
			}
			else
			{
				// release reference
				VirtualKeyboardWidget.Reset();
			}
		}
	}
	else
	{
		// release reference
		VirtualKeyboardWidget.Reset();
	}
}

//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeVirtualKeyboardResult(bool update, String contents);"
void JNICALL UE::Jni::FGameActivity::nativeVirtualKeyboardResult(JNIEnv* jenv, jobject thiz, jboolean update, jstring contents)
{
	::nativeVirtualKeyboardResult(jenv, thiz, update, contents, ETextEntryType::TextEntryAccepted);
}

void JNICALL UE::Jni::FGameActivity::nativeVirtualKeyboardDismissed(JNIEnv* jenv, jobject thiz, jboolean update, jstring contents)
{
	::nativeVirtualKeyboardResult(jenv, thiz, update, contents, ETextEntryType::TextEntryCanceled);
}

//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeVirtualKeyboardChanged(String contents);"
void JNICALL UE::Jni::FGameActivity::nativeVirtualKeyboardChanged(JNIEnv* jenv, jobject thiz, jstring contents)
{
	if (VirtualKeyboardWidget.IsValid())
	{
		FString Contents = FJavaHelper::FStringFromParam(jenv, contents);
		
		// call to set the widget text on game thread
		if (FTaskGraphInterface::IsRunning())
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady([Contents=Contents]()
			{
				TSharedPtr<IVirtualKeyboardEntry> LockedKeyboardWidget(VirtualKeyboardWidget.Pin());
				if (LockedKeyboardWidget.IsValid())
				{
					LockedKeyboardWidget->SetTextFromVirtualKeyboard(FText::FromString(Contents), ETextEntryType::TextEntryUpdated);
				}
			}, TStatId(), NULL, ENamedThreads::GameThread);
		}
	}
}

void JNICALL UE::Jni::FGameActivity::nativeVirtualKeyboardSendKey(JNIEnv* jenv, jobject thiz, jint keyCode)
{
	if (FTaskGraphInterface::IsRunning())
	{
		// Dispatch when ready to delay the submit when the text is being changed by another task (e.g UE::Jni::FGameActivity::nativeVirtualKeyboardChanged)
		FFunctionGraphTask::CreateAndDispatchWhenReady([keyCode = keyCode, CurrentVirtualKeyboardWidget = VirtualKeyboardWidget]()
		{
			if (CurrentVirtualKeyboardWidget.IsValid() && CurrentVirtualKeyboardWidget == VirtualKeyboardWidget)
			{
				FDeferredAndroidMessage Message;

				Message.messageType = MessageType_KeyDown;
				Message.KeyEventData.keyId = keyCode;
				FAndroidInputInterface::DeferMessage(Message);
			}
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}

void JNICALL UE::Jni::FGameActivity::nativeVirtualKeyboardSendSelection(JNIEnv* jenv, jobject thiz, jint selStart, jint selEnd)
{
	// call to set the widget selection on game thread
	if (VirtualKeyboardWidget.IsValid())
	{
		if (FTaskGraphInterface::IsRunning())
		{
			FGraphEventRef SetWidgetSelection = FFunctionGraphTask::CreateAndDispatchWhenReady([&, selStart, selEnd]()
			{
				TSharedPtr<IVirtualKeyboardEntry> LockedKeyboardWidget(VirtualKeyboardWidget.Pin());
				if (LockedKeyboardWidget.IsValid())
				{
					LockedKeyboardWidget->SetSelectionFromVirtualKeyboard(selStart, selEnd);
				}
			}, TStatId(), NULL, ENamedThreads::GameThread);
		}
	}
}

void AndroidThunkCpp_OnAudioFocusChangedListenersUpdated(bool bInHasListeners)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_OnAudioFocusChangedListenersUpdated, bInHasListeners);
	}
}

void AndroidThunkCpp_LaunchURL(const FString& URL)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto Argument = FJavaHelper::ToJavaString(Env, URL);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_LaunchURL, *Argument);
	}
}

void AndroidThunkCpp_ShowAdBanner(const FString& AdUnitID, bool bShowOnBottomOfScreen)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
 	{
		auto AdUnitIDArg = FJavaHelper::ToJavaString(Env, AdUnitID);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_ShowAdBanner, *AdUnitIDArg, bShowOnBottomOfScreen);
	}
}

void AndroidThunkCpp_HideAdBanner()
{
 	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
 	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_HideAdBanner);
 	}
}

void AndroidThunkCpp_CloseAdBanner()
{
 	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
 	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_CloseAdBanner);
 	}
}

void AndroidThunkCpp_LoadInterstitialAd(const FString& AdUnitID)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto AdUnitIDArg = FJavaHelper::ToJavaString(Env, AdUnitID);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_LoadInterstitialAd, *AdUnitIDArg);
	}
}

bool AndroidThunkCpp_IsInterstitialAdAvailable()
{
	bool bIsAdAvailable = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		bIsAdAvailable = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IsInterstitialAdAvailable);
	}

	return bIsAdAvailable;
}

bool AndroidThunkCpp_IsInterstitialAdRequested()
{
	bool bIsAdRequested = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		bIsAdRequested = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IsInterstitialAdRequested);
	}

	return bIsAdRequested;
}

void AndroidThunkCpp_ShowInterstitialAd()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_ShowInterstitialAd);
	}
}

FString AndroidThunkCpp_GetAdvertisingId()
{
	FString adIdResult = FString();

	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		adIdResult = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_GetAdvertisingId));
	}
	return adIdResult;
}

FString AndroidThunkCpp_GetAndroidId()
{
	FString androidIdResult = FString();

	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		androidIdResult = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetAndroidId));
	}
	return androidIdResult;
}

void AndroidThunkCpp_ShareURL(const FString& URL, const FText& Description, const FText& SharePrompt, int32 LocationHintX, int32 LocationHintY)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto URLArg = FJavaHelper::ToJavaString(Env, URL);
		auto DescArg = FJavaHelper::ToJavaString(Env, Description.ToString());
		auto PromptArg = FJavaHelper::ToJavaString(Env, SharePrompt.ToString());
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ShareURL, *URLArg, *DescArg, *PromptArg, LocationHintX, LocationHintY);
	}
}

bool AndroidThunkCpp_IsPackageInstalled(const FString& PackageName)
{
	bool result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto PackageNameArg = FJavaHelper::ToJavaString(Env, PackageName);
		result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IsPackageInstalled, *PackageNameArg);
	}
	return result;
}

bool AndroidThunkCpp_LaunchPackage(const FString& PackageName, const FString& ExtraKey, const FString& ExtraValue)
{
	bool result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto PackageNameArg = FJavaHelper::ToJavaString(Env, PackageName);
		auto ExtraKeyArg = FJavaHelper::ToJavaString(Env, ExtraKey);
		auto ExtraValueArg = FJavaHelper::ToJavaString(Env, ExtraValue);
		result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_LaunchPackage, *PackageNameArg, *ExtraKeyArg, *ExtraValueArg);
	}
	return result;
}

bool AndroidThunkCpp_SendBroadcast(const FString& PackageName, const FString& ExtraKey, const FString& ExtraValue, bool bExit)
{
	bool result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto PackageNameArg = FJavaHelper::ToJavaString(Env, PackageName);
		auto ExtraKeyArg = FJavaHelper::ToJavaString(Env, ExtraKey);
		auto ExtraValueArg = FJavaHelper::ToJavaString(Env, ExtraValue);
		
		result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_SendBroadcast, *PackageNameArg, *ExtraKeyArg, *ExtraValueArg, bExit);
	}
	return result;
}

namespace
{
	jobject GJavaAssetManager = NULL;
	AAssetManager* GAssetManagerRef = NULL;
}

jobject AndroidJNI_GetJavaAssetManager()
{
	if (!GJavaAssetManager)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			auto local = NewScopedJavaObject(Env, FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetAssetManager));
			GJavaAssetManager = (jobject)Env->NewGlobalRef(*local);
			checkf(GJavaAssetManager != nullptr, TEXT("Failed get GJavaAssetManager!"));
		}
		else
		{
			checkf(FAndroidApplication::GetJavaEnv() != nullptr, TEXT("Failed get FAndroidApplication::GetJavaEnv() "));
		}
	}
	return GJavaAssetManager;
}

AAssetManager * AndroidThunkCpp_GetAssetManager()
{
	if (!GAssetManagerRef)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			jobject JavaAssetMgr = AndroidJNI_GetJavaAssetManager();
			GAssetManagerRef = AAssetManager_fromJava(Env, JavaAssetMgr);
		}
	}

	return GAssetManagerRef;
}

void AndroidThunkCpp_Minimize()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_Minimize);
	}
}

void AndroidThunkCpp_ForceQuit()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ForceQuit);
	}
}

void AndroidThunkCpp_SetOrientation(int32 Value)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_SetOrientation, Value);
	}
}

bool AndroidThunkCpp_HasSharedPreference(const FString& Group, const FString& Key)
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_HasSharedPreference, *GroupArg, *KeyArg);
	}
	return Result;
}

void AndroidThunkCpp_SetSharedPreferenceBool(const FString& Group, const FString& Key, bool Value)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_SetSharedPreferenceBoolean, *GroupArg, *KeyArg, Value);
	}
}

bool AndroidThunkCpp_GetSharedPreferenceBool(const FString& Group, const FString& Key, bool DefaultValue)
{
	bool Result = DefaultValue;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetSharedPreferenceBoolean, *GroupArg, *KeyArg, DefaultValue);
	}
	return Result;
}

bool AndroidThunkCpp_GetSharedPreferenceBoolTypeSafe(const FString& Group, const FString& Key, bool& OutValue)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		bool bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetSharedPreferenceBoolean, *GroupArg, *KeyArg, false);

		// SharedPreferences.getBoolean will throw an exception if there is a type mismatch
		if (!Env->ExceptionCheck())
		{
			OutValue = bResult;
			return true;
		}
		else
		{
			Env->ExceptionClear();
			return false;
		}
	}

	return false;
}

void AndroidThunkCpp_SetSharedPreferenceInt(const FString& Group, const FString& Key, int32 Value)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_SetSharedPreferenceInt, *GroupArg, *KeyArg, Value);
	}
}

int32 AndroidThunkCpp_GetSharedPreferenceInt(const FString& Group, const FString& Key, int32 DefaultValue)
{
	int32 Result = DefaultValue;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		Result = FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetSharedPreferenceInt, *GroupArg, *KeyArg, DefaultValue);
	}
	return Result;
}

bool AndroidThunkCpp_GetSharedPreferenceIntTypeSafe(const FString& Group, const FString& Key, int32& OutValue)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		int32 Result = FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetSharedPreferenceInt, *GroupArg, *KeyArg, 0);

		// SharedPreferences.getInt will throw an exception if there is a type mismatch
		if (!Env->ExceptionCheck())
		{
			OutValue = Result;
			return true;
		}
		else
		{
			Env->ExceptionClear();
			return false;
		}
	}

	return false;
}

void AndroidThunkCpp_SetSharedPreferenceString(const FString& Group, const FString& Key, const FString& Value)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		auto ValueArg = FJavaHelper::ToJavaString(Env, Value);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_SetSharedPreferenceString, *GroupArg, *KeyArg, *ValueArg);
	}
}

FString AndroidThunkCpp_GetSharedPreferenceString(const FString& Group, const FString& Key, const FString& DefaultValue)
{
	FString Result = DefaultValue;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		auto DefaultValueArg = FJavaHelper::ToJavaString(Env, DefaultValue);
		Result = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetSharedPreferenceString, *GroupArg, *KeyArg, *DefaultValueArg));
	}
	return Result;
}

FString AndroidThunkCpp_GetCacheDir()
{
	FString Result;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetCacheDir));
	}
	return Result;
}

bool AndroidThunkCpp_GetSharedPreferenceStringTypeSafe(const FString& Group, const FString& Key, FString& OutValue)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		auto ResultObject = FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetSharedPreferenceString, *GroupArg, *KeyArg, NULL);

		// SharedPreferences.getString will throw an exception if there is a type mismatch
		if (Env->ExceptionCheck())
		{
			Env->ExceptionClear();
			return false;
		}
		else if (ResultObject && !Env->IsSameObject(ResultObject, NULL))
		{
			OutValue = FJavaHelper::FStringFromLocalRef(Env, (jstring)ResultObject);
			return true;
		}
	}

	return false;
}

void AndroidThunkCpp_DeleteSharedPreference(const FString& Group, const FString& Key)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		auto KeyArg = FJavaHelper::ToJavaString(Env, Key);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_DeleteSharedPreference, *GroupArg, *KeyArg);
	}
}

void AndroidThunkCpp_DeleteSharedPreferenceGroup(const FString& Group)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto GroupArg = FJavaHelper::ToJavaString(Env, Group);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_DeleteSharedPreferenceGroup, *GroupArg);
	}
}

bool AndroidThunkCpp_IsMusicActive()
{
	bool bIsActive = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		bIsActive = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IsMusicActive);
	}

	return bIsActive;
}

void AndroidThunkCpp_Iap_SetupIapService(const FString& InProductKey, bool InEnableAlternativeBillingOnly)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		auto ProductKey = FJavaHelper::ToJavaString(Env, InProductKey);
		jboolean EnableAlternativeBillingOnly = InEnableAlternativeBillingOnly ? JNI_TRUE : JNI_FALSE;
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IapSetupService, *ProductKey, EnableAlternativeBillingOnly);
	}
}

bool AndroidThunkCpp_Iap_QueryInAppPurchases(const TArray<FString>& ProductIDs)
{
	FPlatformMisc::LowLevelOutputDebugString(TEXT("[JNI] - AndroidThunkCpp_Iap_QueryInAppPurchases"));
	bool bResult = false;

	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		CHECK_JNI_METHOD(FJavaWrapper::AndroidThunkJava_IapQueryInAppPurchases);

		// Populate some java types with the provided product information
		auto ProductIDArray = NewScopedJavaObject(Env, (jobjectArray)Env->NewObjectArray(ProductIDs.Num(), FJavaWrapper::JavaStringClass, NULL));
		if (ProductIDArray)
		{
			for (uint32 Param = 0; Param < ProductIDs.Num(); Param++)
			{
				auto StringValue = FJavaHelper::ToJavaString(Env, ProductIDs[Param]);
				Env->SetObjectArrayElement(*ProductIDArray, Param, *StringValue);
			}
			
			// Execute the java code for this operation
			bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IapQueryInAppPurchases, *ProductIDArray);
		}
	}

	return bResult;
}

bool AndroidThunkCpp_Iap_QueryInAppPurchases(const TArray<FString>& ProductIDs, const TArray<bool>& bConsumable)
{
	FPlatformMisc::LowLevelOutputDebugString(TEXT("AndroidThunkCpp_Iap_QueryInAppPurchases DEPRECATED, won't use consumables array"));
	return AndroidThunkCpp_Iap_QueryInAppPurchases(ProductIDs);
}

bool AndroidThunkCpp_Iap_BeginPurchase(const TArray<FStringView>& ProductIds, const FString& AccountId)
{
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_BeginPurchase"));
	bool bResult = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		CHECK_JNI_METHOD(FJavaWrapper::AndroidThunkJava_IapBeginPurchase);

		auto ProductIdsJava = FJavaHelper::ToJavaStringArray(Env, ProductIds);
		if (AccountId.IsEmpty())
		{
			bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IapBeginPurchase, *ProductIdsJava, nullptr);
		}
		else
		{
			auto ObfuscatedAccountIdJava = FJavaHelper::ToJavaString(Env, AccountId);
			bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IapBeginPurchase, *ProductIdsJava, *ObfuscatedAccountIdJava);
		}
	}

	return bResult;
}

bool AndroidThunkCpp_Iap_ConsumePurchase(const FString& ProductToken)
{
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_ConsumePurchase %s"), *ProductToken);
	
	bool bResult = false;
	if (!ProductToken.IsEmpty())
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			CHECK_JNI_METHOD(FJavaWrapper::AndroidThunkJava_IapConsumePurchase);

			auto ProductTokenJava = FJavaHelper::ToJavaString(Env, ProductToken);
			//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_ConsumePurchase BEGIN"));
			bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IapConsumePurchase, *ProductTokenJava);
			//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_ConsumePurchase END"));
		}
	}

	return bResult;
}

bool AndroidThunkCpp_Iap_AcknowledgePurchase(const FString& ProductToken)
{
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_AcknowledgePurchase %s"), *ProductToken);
	
	bool bResult = false;
	if (!ProductToken.IsEmpty())
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			CHECK_JNI_METHOD(FJavaWrapper::AndroidThunkJava_IapAcknowledgePurchase);

			auto ProductTokenJava = FJavaHelper::ToJavaString(Env, ProductToken);
			//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_AcknowledgePurchase BEGIN"));
			bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IapAcknowledgePurchase, *ProductTokenJava);
			//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_AcknowledgePurchase END"));
		}
	}

	return bResult;
}

bool AndroidThunkCpp_Iap_QueryExistingPurchases()
{
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_QueryExistingPurchases"));
	
	bool bResult = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		CHECK_JNI_METHOD(FJavaWrapper::AndroidThunkJava_IapQueryExistingPurchases);

		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_QueryExistingPurchases BEGIN"));
		bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IapQueryExistingPurchases);
		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[JNI] - AndroidThunkCpp_Iap_QueryExistingPurchases END"));
	}

	return bResult;
}

bool AndroidThunkCpp_Iap_IsAllowedToMakePurchases()
{
	FPlatformMisc::LowLevelOutputDebugString(TEXT("[JNI] - AndroidThunkCpp_Iap_IsAllowedToMakePurchases"));
	bool bResult = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		CHECK_JNI_METHOD(FJavaWrapper::AndroidThunkJava_IapIsAllowedToMakePurchases);

		bResult = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GoogleServicesThis, FJavaWrapper::AndroidThunkJava_IapIsAllowedToMakePurchases);
	}
	return bResult;
}

FString AndroidThunkJava_IapGetBillingCountryCode()
{
	FString Result = FString();
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IapGetBillingCountryCode));
	}

	return Result;
}

bool AndroidThunkJava_IapGetIsAlternativeBillingOnlyAvailable()
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		CHECK_JNI_METHOD(FJavaWrapper::AndroidThunkJava_IapGetIsAlternativeBillingOnlyAvailable)

		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IapGetIsAlternativeBillingOnlyAvailable);
	}

	return Result;
}

bool AndroidThunkJava_IapShowAlternativeBillingOnlyInformationDialog()
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		CHECK_JNI_METHOD(FJavaWrapper::AndroidThunkJava_IapShowAlternativeBillingOnlyInformationDialog)

		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_IapShowAlternativeBillingOnlyInformationDialog);
	}

	return Result;
}

void AndroidThunkCpp_UseSurfaceViewWorkaround()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_UseSurfaceViewWorkaround);
	}
}

void AndroidThunkCpp_SetDesiredViewSize(int32 Width, int32 Height)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_SetDesiredViewSize, Width, Height);
	}

// #if BUILD_EMBEDDED_APP

	// also send info to native wrapper around embedded
	FEmbeddedCallParamsHelper Helper;
	Helper.Command = TEXT("setueresolution");
	Helper.Parameters = { {TEXT("width"), LexToString(Width)}, {TEXT("height"), LexToString(Height)} };
	FEmbeddedDelegates::GetEmbeddedToNativeParamsDelegateForSubsystem(TEXT("native")).Broadcast(Helper);

// #endif
}

int32 AndroidThunkCpp_ScheduleLocalNotificationAtTime(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent, int32 IdOverride)
{
	//Convert FireDateTime to yyyy-MM-dd HH:mm:ss in order to pass to java
	FString FireDateTimeFormatted = FString::FromInt(FireDateTime.GetYear()) + "-" + FString::FromInt(FireDateTime.GetMonth()) + "-" + FString::FromInt(FireDateTime.GetDay()) + " " + FString::FromInt(FireDateTime.GetHour()) + ":" + FString::FromInt(FireDateTime.GetMinute()) + ":" + FString::FromInt(FireDateTime.GetSecond());

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (Env != NULL)
	{
		auto jFireDateTime = FJavaHelper::ToJavaString(Env, FireDateTimeFormatted);
		auto jTitle = FJavaHelper::ToJavaString(Env, Title.ToString());
		auto jBody = FJavaHelper::ToJavaString(Env, Body.ToString());
		auto jAction = FJavaHelper::ToJavaString(Env, Action.ToString());
		auto jActivationEvent = FJavaHelper::ToJavaString(Env, ActivationEvent);
		
		return FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_LocalNotificationScheduleAtTime, *jFireDateTime, LocalTime, *jTitle, *jBody, *jAction, *jActivationEvent, IdOverride);
	}
	
	return -1;
}

void AndroidThunkCpp_GetLaunchNotification(bool& NotificationLaunchedApp, FString& ActivationEvent, int32& FireDate)
{
	bool bIsOptional = false;

	NotificationLaunchedApp = false;
	ActivationEvent = "";
	FireDate = 0;

	JNIEnv* Env = FAndroidApplication::GetJavaEnv();
	if (Env != NULL)
	{
		auto launchInfo = NewScopedJavaObject(Env, (jobject)Env->CallObjectMethod(FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_LocalNotificationGetLaunchNotification));
		if (launchInfo && FJavaWrapper::LaunchNotificationUsed != 0)
		{
			NotificationLaunchedApp = (bool)Env->GetBooleanField(*launchInfo, FJavaWrapper::LaunchNotificationUsed);

			ActivationEvent = FJavaHelper::FStringFromLocalRef(Env, (jstring)Env->GetObjectField(*launchInfo, FJavaWrapper::LaunchNotificationEvent));
			
			FireDate = (int32)Env->GetIntField(*launchInfo, FJavaWrapper::LaunchNotificationFireDate);
		}
	}
}

void AndroidThunkCpp_ClearAllLocalNotifications() 
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_LocalNotificationClearAll);
	}
}

bool AndroidThunkCpp_LocalNotificationExists(int32 NotificationId)
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_LocalNotificationExists, NotificationId);
	}
	return Result;
}

bool AndroidThunkCpp_DestroyScheduledNotificationIfExists(int32 NotificationId)
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_LocalNotificationDestroyIfExists, NotificationId);
	}
	return Result;
}

int32 AndroidThunkCpp_GetNetworkConnectionType()
{
	int32 result = -1;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		result = FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetNetworkConnectionType);
	}

	return result;
}

bool AndroidThunkCpp_AcquireWifiManagerMulticastLock()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		return FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_AcquireWifiManagerMulticastLock);
	}
	return false;
}

void AndroidThunkCpp_ReleaseWifiManagerMulticastLock()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ReleaseWifiManagerMulticastLock);
	}	
}

//The JNI_OnLoad function is triggered by loading the game library from 
//the Java source file.
//	static
//	{
//		System.loadLibrary("MyGame");
//	}
//
// Use the JNI_OnLoad function to map all the class IDs and method IDs to their respective
// variables. That way, later when the Java functions need to be called, the IDs will be ready.
// It is much slower to keep looking up the class and method IDs.
JNIEXPORT jint JNI_OnLoad(JavaVM* InJavaVM, void* InReserved)
{
	FTaskTagScope::SetTagNone();
	
	FPlatformMisc::LowLevelOutputDebugString(TEXT("In the JNI_OnLoad function"));

	JNIEnv* Env = NULL;
	InJavaVM->GetEnv((void **)&Env, JNI_CURRENT_VERSION);

	// if you have problems with stuff being missing especially in distribution builds then it could be because proguard is stripping things from java
	// check proguard-project.txt and see if your stuff is included in the exceptions
	GJavaVM = InJavaVM;
	FAndroidApplication::InitializeJavaEnv();

	FJavaWrapper::FindClassesAndMethods(Env);

	// hook signals
	if (!FPlatformMisc::IsDebuggerPresent() || GAlwaysReportCrash)
	{
		// disable crash handler.. getting better stack traces from system for now
		//FPlatformMisc::SetCrashHandler(EngineCrashHandler);
	}

	// Cache path to external storage
	auto EnvClass = NewScopedJavaObject(Env, Env->FindClass("android/os/Environment"));
	jmethodID getExternalStorageDir = Env->GetStaticMethodID(*EnvClass, "getExternalStorageDirectory", "()Ljava/io/File;");
	auto externalStoragePath = NewScopedJavaObject(Env, Env->CallStaticObjectMethod(*EnvClass, getExternalStorageDir, nullptr));
	jmethodID getFilePath = Env->GetMethodID(Env->FindClass("java/io/File"), "getPath", "()Ljava/lang/String;");
	// Copy that somewhere safe
	GFilePathBase = FJavaHelper::FStringFromLocalRef(Env, (jstring)Env->CallObjectMethod(*externalStoragePath, getFilePath, nullptr));
	GOBBFilePathBase = GFilePathBase;
	GOBBMainFilePath = TEXT("");
	GOBBPatchFilePath = TEXT("");
	GOBBOverflow1FilePath = TEXT("");
	GOBBOverflow2FilePath = TEXT("");

	// then release...
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Path found as '%s'\n"), *GFilePathBase);

	// Get the system font directory
	GFontPathBase = FJavaHelper::FStringFromLocalRef(Env, (jstring)Env->CallStaticObjectMethod(FJavaWrapper::GameActivityClassID, FJavaWrapper::AndroidThunkJava_GetFontDirectory));
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Font Path found as '%s'\n"), *GFontPathBase);

	// Wire up to core delegates, so core code can call out to Java
	DECLARE_DELEGATE_OneParam(FAndroidLaunchURLDelegate, const FString&);
	extern CORE_API FAndroidLaunchURLDelegate OnAndroidLaunchURL;
	OnAndroidLaunchURL = FAndroidLaunchURLDelegate::CreateStatic(&AndroidThunkCpp_LaunchURL);

	FPlatformMisc::LowLevelOutputDebugString(TEXT("In the JNI_OnLoad function 5"));

	if (InitWebRTCOnAndroid)
	{
		InitWebRTCOnAndroid(GJavaVM);
	}
	
	return JNI_CURRENT_VERSION;
}

//Native-defined functions

//This function is declared in the Java-defined class, GameActivity.java: "public native void naativeSetObbFilePaths();"
void JNICALL UE::Jni::FGameActivity::nativeSetObbFilePaths(JNIEnv* jenv, jobject thiz, jstring OBBMainFilePath, jstring OBBPatchFilePath, jstring OBBOverflow1FilePath, jstring OBBOverflow2FilePath)
{
	GOBBMainFilePath = FJavaHelper::FStringFromParam(jenv, OBBMainFilePath);
	GOBBPatchFilePath = FJavaHelper::FStringFromParam(jenv, OBBPatchFilePath);
	GOBBOverflow1FilePath = FJavaHelper::FStringFromParam(jenv, OBBOverflow1FilePath);
	GOBBOverflow2FilePath = FJavaHelper::FStringFromParam(jenv, OBBOverflow2FilePath);
}

//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeSetGlobalActivity();"
void JNICALL UE::Jni::FGameActivity::nativeSetGlobalActivity(JNIEnv* jenv, jobject thiz, jboolean bUseExternalFilesDir, jboolean bPublicLogFiles, jstring internalFilePath, jstring externalFilePath, jboolean bOBBinAPK, jstring APKFilename)
{
	STANDALONE_DEBUG_LOG(TEXT("nativeSetGlobalActivity(unreal): Entering unreal nativeSetGlobalActivity, GameActivityThis=%p\n"), FJavaWrapper::GameActivityThis);
	if (FJavaWrapper::GameActivityThis != nullptr)
	{
		STANDALONE_DEBUG_LOG(TEXT("nativeSetGlobalActivity(unreal): Error GameActivityThis is already set GameActivityThis=%p\n"), FJavaWrapper::GameActivityThis);
		//jenv->DeleteGlobalRef(FJavaWrapper::GameActivityThis);
		FJavaWrapper::GameActivityThis = nullptr;
	}
	if (!FJavaWrapper::GameActivityThis)
	{
		FJavaWrapper::GameActivityThis = jenv->NewGlobalRef(thiz);
		if (!FJavaWrapper::GameActivityThis)
		{
			FPlatformMisc::LowLevelOutputDebugString(TEXT("Error setting the global GameActivity activity"));
			check(false);
		}
		STANDALONE_DEBUG_LOG(TEXT("nativeSetGlobalActivity(unreal): jenv=%p, set GameActivityThis=%p, bUseExternalFilesDir=%d, bOBBinAPK=%d\n"), jenv, FJavaWrapper::GameActivityThis, bUseExternalFilesDir, bOBBinAPK);

		// @todo split GooglePlay, this needs to be passed in to this function
		FJavaWrapper::GoogleServicesThis = FJavaWrapper::GameActivityThis;
		// FJavaWrapper::GoogleServicesThis = jenv->NewGlobalRef(googleServices);

		// Next we check to see if the OBB file is in the APK
		//jmethodID isOBBInAPKMethod = jenv->GetStaticMethodID(FJavaWrapper::GameActivityClassID, "isOBBInAPK", "()Z");
		//GOBBinAPK = (bool)jenv->CallStaticBooleanMethod(FJavaWrapper::GameActivityClassID, isOBBInAPKMethod, nullptr);
		GOBBinAPK = bOBBinAPK;

		GAPKFilename = FJavaHelper::FStringFromParam(jenv, APKFilename);
		GInternalFilePath = FJavaHelper::FStringFromParam(jenv, internalFilePath);
		GExternalFilePath = FJavaHelper::FStringFromParam(jenv, externalFilePath);

		if (bUseExternalFilesDir)
		{
#if UE_BUILD_SHIPPING
			GFilePathBase = GInternalFilePath;
			GOverrideAndroidLogDir = bPublicLogFiles;
#else
			GFilePathBase = GExternalFilePath;
#endif
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("nativeSetGlobalActivity(unreal): GFilePathBase Path override to'%s'\n"), *GFilePathBase);
		}

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("nativeSetGlobalActivity(unreal): InternalFilePath found as '%s'\n"), *GInternalFilePath);
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("nativeSetGlobalActivity(unreal): ExternalFilePath found as '%s'\n"), *GExternalFilePath);
	}
}


jboolean JNICALL UE::Jni::FGameActivity::nativeIsShippingBuild(JNIEnv* LocalJNIEnv, jobject LocalThiz)
{
#if UE_BUILD_SHIPPING
	return JNI_TRUE;
#else
	return JNI_FALSE;
#endif
}

void JNICALL UE::Jni::FGameActivity::nativeOnActivityResult(JNIEnv* jenv, jobject thiz, FGameActivity* activity, jint requestCode, jint resultCode, Android::Content::FIntent* data)
{
	FJavaWrapper::OnActivityResultDelegate.Broadcast(jenv, thiz, activity, requestCode, resultCode, data);
}

void JNICALL UE::Jni::FGameActivity::nativeHandleSensorEvents(JNIEnv* jenv, jobject thiz, Java::Lang::TArray<jfloat>* tilt, Java::Lang::TArray<jfloat>* rotation_rate, Java::Lang::TArray<jfloat>* gravity, Java::Lang::TArray<jfloat>* acceleration)
{
	jfloat* tiltFloatValues = jenv->GetFloatArrayElements(tilt, 0);
	FVector current_tilt(tiltFloatValues[0], tiltFloatValues[1], tiltFloatValues[2]);
	jenv->ReleaseFloatArrayElements(tilt, tiltFloatValues, 0);

	jfloat* rotation_rate_FloatValues = jenv->GetFloatArrayElements(rotation_rate, 0);
	FVector current_rotation_rate(rotation_rate_FloatValues[0], rotation_rate_FloatValues[1], rotation_rate_FloatValues[2]);
	jenv->ReleaseFloatArrayElements(rotation_rate, rotation_rate_FloatValues, 0);

	jfloat* gravity_FloatValues = jenv->GetFloatArrayElements(gravity, 0);
	FVector current_gravity(gravity_FloatValues[0], gravity_FloatValues[1], gravity_FloatValues[2]);
	jenv->ReleaseFloatArrayElements(gravity, gravity_FloatValues, 0);
	
	jfloat* acceleration_FloatValues = jenv->GetFloatArrayElements(acceleration, 0);
	FVector current_acceleration(acceleration_FloatValues[0], acceleration_FloatValues[1], acceleration_FloatValues[2]);
	jenv->ReleaseFloatArrayElements(acceleration, acceleration_FloatValues, 0);

	FAndroidInputInterface::QueueMotionData(current_tilt, current_rotation_rate, current_gravity, current_acceleration);

}

void AndroidThunkCpp_ClipboardCopy(const FString& Str)
{
    if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
    {
		auto JStr = FJavaHelper::ToJavaString(Env, Str);
        FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ClipboardCopy, *JStr);
    }
}

FString AndroidThunkCpp_ClipboardPaste()
{
    FString PasteStringResult = FString();
    
    if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
    {
        PasteStringResult = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_ClipboardPaste));
    }
    return PasteStringResult;
}

TArray<int32> AndroidThunkCpp_GetSupportedNativeDisplayRefreshRates()
{
	TArray<int32> Result;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		jintArray JavaIntArray = (jintArray)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetSupportedNativeDisplayRefreshRates);
		if (JavaIntArray != nullptr)
		{
			jint* JavaInts = Env->GetIntArrayElements(JavaIntArray, 0);
			if (JavaInts)
			{
				jsize IntsLen = Env->GetArrayLength(JavaIntArray);
				Result.Empty(IntsLen);
				for (int32 Idx = 0; Idx < IntsLen; Idx++)
				{
					Result.Add(JavaInts[Idx]);
				}
				Env->ReleaseIntArrayElements(JavaIntArray, JavaInts, 0);
			}
			Env->DeleteLocalRef(JavaIntArray);
		}
	}
	return Result;
}

void AndroidThunkJava_AddNetworkListener()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_AddNetworkListener);
	}
}

void AndroidThunkJava_RemoveNetworkListener()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_RemoveNetworkListener);
	}
}

bool AndroidThunkCpp_SetNativeDisplayRefreshRate(int32 RefreshRate)
{
	bool Result = false;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_SetNativeDisplayRefreshRate, RefreshRate);
	}
	return Result;
}

int32 AndroidThunkCpp_GetNativeDisplayRefreshRate()
{
	int32 Result = 60;
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		Result = FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_GetNativeDisplayRefreshRate);
	}
	return Result;
}

void AndroidThunkCpp_EnableMotion(bool bEnable)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// call the java side
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, FJavaWrapper::AndroidThunkJava_EnableMotion, bEnable);
	}
}

JNI_METHOD void Java_com_epicgames_unreal_GameActivity_nativeOnSafetyNetAttestationSucceeded(JNIEnv* jenv, jobject thiz, jstring jwsData)
{
	FString JwsString = FJavaHelper::FStringFromParam(jenv, jwsData);

	// call to OnSafetyNetAttestationResultDelegate on game thread
	if (FTaskGraphInterface::IsRunning())
	{
		FGraphEventRef SafetyNetAttestationSucceeded = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
		{
			FJavaWrapper::OnSafetyNetAttestationResultDelegate.Broadcast(true, JwsString, 0);
		}, TStatId(), NULL, ENamedThreads::GameThread);
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(SafetyNetAttestationSucceeded);
	}
}

JNI_METHOD void Java_com_epicgames_unreal_GameActivity_nativeOnSafetyNetAttestationFailed(JNIEnv* jenv, jobject thiz, jint jwsValue)
{
	// call to OnSafetyNetAttestationResultDelegate on game thread
	if (FTaskGraphInterface::IsRunning())
	{
		FGraphEventRef SafetyNetAttestationFailed = FFunctionGraphTask::CreateAndDispatchWhenReady([&]()
		{
			FJavaWrapper::OnSafetyNetAttestationResultDelegate.Broadcast(false, FString(), jwsValue);
		}, TStatId(), NULL, ENamedThreads::GameThread);
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(SafetyNetAttestationFailed);
	}
}

#include "Async/TaskGraphInterfaces.h"

static jmethodID ReplyMethod;

void AndroidThunkCpp_OnNativeToEmbeddedReply(FString ID, const FEmbeddedCommunicationMap& InReturnValues, FString InError, FString RoutingFunction)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		// marshall some data
#if !UE_BUILD_SHIPPING
		UE_LOGF(LogInit, Display, "Java call Id: %ls, Routing Function: %ls, Error %ls, Params:", *ID, *RoutingFunction, *InError);
#endif

		// create a string array for the pairs
		static auto StringClass = FJavaWrapper::FindClassGlobalRef(Env, "java/lang/String", false);
		auto ReturnValues = NewScopedJavaObject(Env, Env->NewObjectArray(InReturnValues.Num() * 2, StringClass, 0));
		int32 Index = 0;
		for (auto It : InReturnValues)
		{
			auto KeyString = FJavaHelper::ToJavaString(Env, It.Key);
			auto ValueString = FJavaHelper::ToJavaString(Env, It.Value);
			Env->SetObjectArrayElement(*ReturnValues, Index++, *KeyString);
			Env->SetObjectArrayElement(*ReturnValues, Index++, *ValueString);
#if !UE_BUILD_SHIPPING
			UE_LOGF(LogInit, Display, "  %ls : %ls", *It.Key, *It.Value);
#endif
		}
		auto Error = FJavaHelper::ToJavaString(Env, InError);
		
		// call back into java
		auto IDReturn = FJavaHelper::ToJavaString(Env, ID);
		auto RoutingFunctionReturn = FJavaHelper::ToJavaString(Env, RoutingFunction);
		
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, ReplyMethod, *IDReturn, *ReturnValues, *Error, *RoutingFunctionReturn);
	}
}



void JNICALL UE::Jni::FNativeCalls::CallNativeToEmbedded(JNIEnv* jenv, jclass clazz, jstring InID, jint Priority, jstring InSubsystem, jstring InCommand, Java::Lang::TArray<jstring>* InParams, jstring InRoutingFunction)
{
#if BUILD_EMBEDDED_APP
	auto Subsystem = FJavaHelper::FStringFromParam(jenv, InSubsystem);
	auto Command = FJavaHelper::FStringFromParam(jenv, InCommand);
	auto ID = FJavaHelper::FStringFromParam(jenv, InID);
	auto RoutingFunction = FJavaHelper::FStringFromParam(jenv, InRoutingFunction);
	
	FEmbeddedCallParamsHelper Helper;
	Helper.Command = Command;

	FName SubsystemName(*Subsystem);
	FName CommandName = *FString::Printf(TEXT("%s_%s"), *Subsystem, *Command);

	// wake up UE
	FEmbeddedCommunication::KeepAwake(CommandName, false);

#if !UE_BUILD_SHIPPING
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("NativeToEmbeddedCall: Subsystem: %s, Command: %s, Params:"), *Subsystem, *Helper.Command);
#endif

	if (InParams != nullptr)
	{
		int32 Count = jenv->GetArrayLength(InParams);
		int32 Index = 0;
		while (Index < Count)
		{
			auto javaKey = FJavaHelper::FStringFromLocalRef(jenv, (jstring)(jenv->GetObjectArrayElement(InParams, Index++)));
			auto javaValue = FJavaHelper::FStringFromLocalRef(jenv, (jstring)(jenv->GetObjectArrayElement(InParams, Index++)));

			Helper.Parameters.Add(javaKey, javaValue);
#if !UE_BUILD_SHIPPING
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("  %s : %s"), *javaKey, *javaValue);
#endif
		}
	}
	
	Helper.OnCompleteDelegate = [CommandName, ID, RoutingFunction](const FEmbeddedCommunicationMap& InReturnValues, FString InError)
	{
		// wake up UE
		FEmbeddedCommunication::AllowSleep(CommandName);

		AndroidThunkCpp_OnNativeToEmbeddedReply(ID, InReturnValues, InError, RoutingFunction);
	};

	FEmbeddedCommunication::RunOnGameThread(Priority, [SubsystemName, Helper]()
	{
		if (FEmbeddedDelegates::GetNativeToEmbeddedParamsDelegateForSubsystem(SubsystemName).IsBound())
		{
			FEmbeddedDelegates::GetNativeToEmbeddedParamsDelegateForSubsystem(SubsystemName).Broadcast(Helper);
		}
		else
		{
			Helper.OnCompleteDelegate({} , FString::Printf(TEXT("No one is listening to subsystem %s"), *SubsystemName.ToString()));
		}
	});
#endif // BUILD_EMBEDDED_APP
}

void JNICALL UE::Jni::FNativeCalls::SetNamedObject(JNIEnv* jenv, jclass clazz, jstring InName, jobject InObj)
{
	auto Name = FJavaHelper::FStringFromParam(jenv, InName);
	FEmbeddedDelegates::SetNamedObject(Name, (void*)InObj);
}

void JNICALL UE::Jni::FNativeCalls::KeepAwake(JNIEnv* jenv, jclass clazz, jstring InRequester, jboolean bIsForRendering)
{
	auto Requester = FJavaHelper::FStringFromParam(jenv, InRequester);
	FEmbeddedCommunication::KeepAwake(*Requester, bIsForRendering);
}

void JNICALL UE::Jni::FNativeCalls::AllowSleep(JNIEnv* jenv, jclass clazz, jstring InRequester)
{
	auto Requester = FJavaHelper::FStringFromParam(jenv, InRequester);
	FEmbeddedCommunication::AllowSleep(*Requester);
}

#if !BUILD_EMBEDDED_APP
DEFINE_LOG_CATEGORY_STATIC(LogJava, Log, All);
#endif

void JNICALL UE::Jni::FNativeCalls::UELogError(JNIEnv* jenv, jclass clazz, jstring InString)
{
	const auto chars = jenv->GetStringUTFChars(InString, 0);
#if BUILD_EMBEDDED_APP
	FEmbeddedCommunication::UELogError(UTF8_TO_TCHAR(chars));
#else
	if (GLog && UE_LOG_ACTIVE(LogJava, Error))
	{
		GLog->Log("LogJava", ELogVerbosity::Error, UTF8_TO_TCHAR(chars));
	}
#endif
	jenv->ReleaseStringUTFChars(InString, chars);
}

void JNICALL UE::Jni::FNativeCalls::UELogWarning(JNIEnv* jenv, jclass clazz, jstring InString)
{
	const auto chars = jenv->GetStringUTFChars(InString, 0);
#if BUILD_EMBEDDED_APP
	FEmbeddedCommunication::UELogWarning(UTF8_TO_TCHAR(chars));
#else
	if (GLog && UE_LOG_ACTIVE(LogJava, Warning))
	{
		GLog->Log("LogJava", ELogVerbosity::Warning, UTF8_TO_TCHAR(chars));
	}
#endif
	jenv->ReleaseStringUTFChars(InString, chars);
}

void JNICALL UE::Jni::FNativeCalls::UELogLog(JNIEnv* jenv, jclass clazz, jstring InString)
{
	const auto chars = jenv->GetStringUTFChars(InString, 0);
#if BUILD_EMBEDDED_APP
	FEmbeddedCommunication::UELogLog(UTF8_TO_TCHAR(chars));
#else
	if (GLog && UE_LOG_ACTIVE(LogJava, Log))
	{
		GLog->Log("LogJava", ELogVerbosity::Log, UTF8_TO_TCHAR(chars));
	}
#endif
	jenv->ReleaseStringUTFChars(InString, chars);
}

void JNICALL UE::Jni::FNativeCalls::UELogVerbose(JNIEnv* jenv, jclass clazz, jstring InString)
{
	const auto chars = jenv->GetStringUTFChars(InString, 0);
#if BUILD_EMBEDDED_APP
	FEmbeddedCommunication::UELogVerbose(UTF8_TO_TCHAR(chars));
#else
	if (GLog && UE_LOG_ACTIVE(LogJava, Verbose))
	{
		GLog->Log("LogJava", ELogVerbosity::Verbose, UTF8_TO_TCHAR(chars));
	}
#endif
	jenv->ReleaseStringUTFChars(InString, chars);
}

void FJavaWrapper::SetupEmbeddedCommunication(JNIEnv* Env)
{
#if BUILD_EMBEDDED_APP
	ReplyMethod = FJavaWrapper::FindMethod(Env, GameActivityClassID, "AndroidThunkJava_OnNativeToEmbeddedReply", "(Ljava/lang/Object;[Ljava/lang/String;Ljava/lang/String;Ljava/lang/String;)V", false);

	FEmbeddedDelegates::GetEmbeddedToNativeParamsDelegateForSubsystem(TEXT("webview")).AddLambda([](const FEmbeddedCallParamsHelper& InMessage)
	{
		UE_LOGF(LogAndroid, Display, "Calling out from embedded to native for 'webview' subsystem:");

		AndroidThunkCpp_OnNativeToEmbeddedReply(InMessage.Command, InMessage.Parameters, TEXT(""), TEXT("webviewnotification"));

		// @todo: Do we need to have results passed back? if so we will need a wrapper block that we pass in to all of the above handling things to call back
		// same for ios
		if (InMessage.OnCompleteDelegate != nullptr)
		{
			InMessage.OnCompleteDelegate(FEmbeddedCommunicationMap(), TEXT(""));
		}
	});

	FEmbeddedDelegates::GetEmbeddedToNativeParamsDelegateForSubsystem(TEXT("native")).AddLambda([](const FEmbeddedCallParamsHelper& InMessage)
	{
		UE_LOGF(LogAndroid, Display, "Calling out from embedded to native for 'native' subsystem:");

		AndroidThunkCpp_OnNativeToEmbeddedReply(InMessage.Command, InMessage.Parameters, TEXT(""), TEXT("native"));

		// @todo: Do we need to have results passed back? if so we will need a wrapper block that we pass in to all of the above handling things to call back
		// same for ios
		if (InMessage.OnCompleteDelegate != nullptr)
		{
			InMessage.OnCompleteDelegate(FEmbeddedCommunicationMap(), TEXT(""));
		}
	});
#endif
}

void JNICALL UE::Jni::FNativeCalls::ForwardNotification(JNIEnv* jenv, jclass clazz, jstring payload)
{
	//
}

void JNICALL UE::Jni::FNativeCalls::RouteServiceIntent(JNIEnv* jenv, jclass clazz, jstring InAction, jstring InPayload)
{
	// call to OnSafetyNetAttestationResultDelegate on game thread
	if (FTaskGraphInterface::IsRunning())
	{
		const char *nativeAction = jenv->GetStringUTFChars(InAction, 0);
		FString Action = FString(nativeAction);
		jenv->ReleaseStringUTFChars(InAction, nativeAction);

		const char *nativePayload = jenv->GetStringUTFChars(InPayload, 0);
		FString Payload = FString(nativePayload);
		jenv->ReleaseStringUTFChars(InPayload, nativePayload);

		FGraphEventRef RouteServiceIntentTask = FFunctionGraphTask::CreateAndDispatchWhenReady([Action, Payload]()
		{
			FJavaWrapper::OnRouteServiceIntentDelegate.Broadcast(Action, Payload);
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}

void JNICALL UE::Jni::FGameActivity::nativeOnThermalStatusChangedListener(JNIEnv* jenv, jobject thiz, jint Status)
{
	FAndroidStats::OnThermalStatusChanged(Status);

	FCoreDelegates::ETemperatureSeverity Severity;
	switch (Status)
	{
	case 0:
	case 1:
		Severity = FCoreDelegates::ETemperatureSeverity::Good;
		break;

	case 2:
		Severity = FCoreDelegates::ETemperatureSeverity::Bad;
		break;

	case 3:
	case 4:
		Severity = FCoreDelegates::ETemperatureSeverity::Serious;
		break;

	case 5:
	case 6:
		Severity = FCoreDelegates::ETemperatureSeverity::Critical;
		break;

	default:
		Severity = FCoreDelegates::ETemperatureSeverity::Unknown;
		break;
	}

	FCoreDelegates::OnTemperatureChange.Broadcast(Severity);
}

static void OnTrimMessage(int MemoryTrimValue)
{
	FAndroidPlatformMemory::UpdateOSMemoryStatus(FAndroidPlatformMemory::EOSMemoryStatusCategory::OSTrim, MemoryTrimValue);
	FAndroidStats::OnTrimMemory(MemoryTrimValue);
}

#if !UE_BUILD_SHIPPING
FAutoConsoleCommand TestTrimMessage(
	TEXT("android.TestTrimMessage"),
	TEXT("testing only, android.TestTrimMessage int \n")
	TEXT("int value must match expected values from android OS (see ComponentCallbacks2 api)\n")
	TEXT("eg. android.TestTrimMessage 15")
	,
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if(Args.Num() > 0)
			{
				uint64 MemoryTrimValue = 0;
				LexFromString(MemoryTrimValue, *Args[0]);
				if(MemoryTrimValue)
				{
					OnTrimMessage(MemoryTrimValue);
				}
			}
		}));
#endif

void JNICALL UE::Jni::FGameActivity::nativeOnTrimMemory(JNIEnv* jenv, jobject thiz, jint MemoryTrimValue)
{
	OnTrimMessage(MemoryTrimValue);
}


class FAndroidEmbeddedExec : public FSelfRegisteringExec
{
public:
	FAndroidEmbeddedExec()
		: FSelfRegisteringExec()
	{

	}

	virtual bool Exec(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (FParse::Command(&Cmd, TEXT("Android")))
		{
			// commands to override and append commandline options for next boot (see FIOSCommandLineHelper)
			if (FParse::Command(&Cmd, TEXT("HideWeb")))
			{
				AndroidThunkCpp_OnNativeToEmbeddedReply(TEXT("HideWeb"), {}, TEXT(""), TEXT("native"));
			}
			else if (FParse::Command(&Cmd, TEXT("ShowWeb")))
			{
				AndroidThunkCpp_OnNativeToEmbeddedReply(TEXT("ShowWeb"), {}, TEXT(""), TEXT("native"));
			}
		}

		return false;
	}
} GAndroidEmbeddedExec;



#endif
