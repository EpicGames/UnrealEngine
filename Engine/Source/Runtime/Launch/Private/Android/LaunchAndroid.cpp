// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#if USE_ANDROID_LAUNCH
#include "Containers/StringView.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceError.h"
#include "LaunchEngineLoop.h"
#include <string.h>
#include <pthread.h>
#include "Android/AndroidJNI.h"
#include "Android/AndroidEventManager.h"
#include "Android/AndroidInputInterface.h"
#include "Android/AndroidJniGameActivity.h"
#include "Android/AndroidJniNativeCalls.h"
#include <android/asset_manager.h>
#include <android/asset_manager_jni.h>
#include <android/log.h>
#include <cpu-features.h>
#include <android_native_app_glue.h>
#include <cstdio>
#include <sys/resource.h>
#include <sys/system_properties.h>
#include <dlfcn.h>
#include "Android/AndroidWindow.h"
#include "Android/AndroidDynamicRHI.h"
#include "Android/AndroidApplication.h"
#include "Android/AndroidPlatformStackWalk.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IHeadMountedDisplayModule.h"
#include "ISessionServicesModule.h"
#include "ISessionService.h"
#include "Engine/Engine.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/PlatformInput.h"
#include "HAL/ThreadHeartBeat.h"
#include "Modules/ModuleManager.h"
#include "IMessagingModule.h"
#include "Android/AndroidStats.h"
#include "MoviePlayer.h"
#include "PreLoadScreenManager.h"
#include "Misc/EmbeddedCommunication.h"
#include "Async/Async.h"
#include <android/sensor.h>

#include "HAL/ExceptionHandling.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Android/AndroidPlatformCrashContext.h"
#include "UObject/GarbageCollection.h"

#if USE_ANDROID_STANDALONE
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IPlatformFilePak.h"
#endif

// Function pointer for retrieving joystick events
// Function has been part of the OS since Honeycomb, but only appeared in the
// NDK in r19. Querying via dlsym allows its use without tying to the newest
// NDK.
typedef float(*GetAxesType)(const AInputEvent*, int32_t axis, size_t pointer_index);
static GetAxesType GetAxes = NULL;

// Define missing events for earlier NDKs
#if PLATFORM_ANDROID_NDK_VERSION < 140200
#define AMOTION_EVENT_AXIS_RELATIVE_X 27
#define AMOTION_EVENT_AXIS_RELATIVE_Y 28
#endif

#ifndef ANDROID_ALLOWCUSTOMTOUCHEVENT
#define ANDROID_ALLOWCUSTOMTOUCHEVENT	0
#endif

// List of default axes to query for each controller
// Ideal solution is to call out to Java and enumerate the list of axes.
static const int32_t AxisList[] =
{
	AMOTION_EVENT_AXIS_X,
	AMOTION_EVENT_AXIS_Y,
	AMOTION_EVENT_AXIS_Z,
	AMOTION_EVENT_AXIS_RX,
	AMOTION_EVENT_AXIS_RY,
	AMOTION_EVENT_AXIS_RZ,

	//These are DPAD analogs
	AMOTION_EVENT_AXIS_HAT_X,
	AMOTION_EVENT_AXIS_HAT_Y,
};

// map of all supported keycodes
static TSet<uint32> MappedKeyCodes;

// map of always allowed keycodes
static TSet<uint32> AlwaysAllowedKeyCodes;

// List of always allowed keycodes
static const uint32 AlwaysAllowedKeyCodesList[] =
{
	AKEYCODE_MENU,
	AKEYCODE_BACK,
	AKEYCODE_VOLUME_UP,
	AKEYCODE_VOLUME_DOWN
};

// List of desired gamepad keycodes
static const uint32 ValidGamepadKeyCodesList[] =
{
	AKEYCODE_BUTTON_A,
	AKEYCODE_DPAD_CENTER,
	AKEYCODE_BUTTON_B,
	AKEYCODE_BUTTON_C,
	AKEYCODE_BUTTON_X,
	AKEYCODE_BUTTON_Y,
	AKEYCODE_BUTTON_Z,
	AKEYCODE_BUTTON_L1,
	AKEYCODE_BUTTON_R1,
	AKEYCODE_BUTTON_START,
	AKEYCODE_MENU,
	AKEYCODE_BUTTON_SELECT,
	AKEYCODE_BACK,
	AKEYCODE_BUTTON_THUMBL,
	AKEYCODE_BUTTON_THUMBR,
	AKEYCODE_BUTTON_L2,
	AKEYCODE_BUTTON_R2,
	AKEYCODE_DPAD_UP,
	AKEYCODE_DPAD_DOWN,
	AKEYCODE_DPAD_LEFT,
	AKEYCODE_DPAD_RIGHT,
	3002  // touchpad
};

// map of gamepad keycodes that should be passed forward
static TSet<uint32> ValidGamepadKeyCodes;

// -nostdlib means no crtbegin_so.o, so we have to provide our own __dso_handle and atexit()
// this is not needed now we are using stdlib (later NDK has more functionality we should keep)
#if 0
extern "C"
{
	int atexit(void (*func)(void)) { return 0; }

	extern void *__dso_handle __attribute__((__visibility__ ("hidden")));
	void *__dso_handle;
}
#endif

#if USE_ANDROID_STANDALONE
int32 GAndroidEnableNativeResizeEvent = 0;
#else
int32 GAndroidEnableNativeResizeEvent = 1;
#endif
static FAutoConsoleVariableRef CVarEnableResizeNativeEvent(
	TEXT("Android.EnableNativeResizeEvent"),
	GAndroidEnableNativeResizeEvent,
	TEXT("Whether native resize event is enabled on Android.\n")
	TEXT(" 0: disabled\n")
	TEXT(" 1: enabled (default)"),
	ECVF_ReadOnly);

int32 GAndroidEnableMouse = 0;
static FAutoConsoleVariableRef CVarEnableMouse(
	TEXT("Android.EnableMouse"),
	GAndroidEnableMouse,
	TEXT("Whether mouse support is enabled on Android.\n")
	TEXT(" 0: disabled (default)\n")
	TEXT(" 1: enabled"),
	ECVF_ReadOnly);

int32 GAndroidEnableHardwareKeyboard = 0;
static FAutoConsoleVariableRef CVarEnableHWKeyboard(
	TEXT("Android.EnableHardwareKeyboard"),
	GAndroidEnableHardwareKeyboard,
	TEXT("Whether hardware keyboard support is enabled on Android.\n")
	TEXT(" 0: disabled (default)\n")
	TEXT(" 1: enabled"),
	ECVF_ReadOnly);

int32 GAndroidTerminateOnExit = 0;
static FAutoConsoleVariableRef CVarTerminateOnExit(
	TEXT("Android.TerminateOnExit"),
	GAndroidTerminateOnExit,
	TEXT("Whether process is terminated on exit instead of normal shutdown on Android.\n")
	TEXT(" 0: disabled (default)\n")
	TEXT(" 1: enabled"),
	ECVF_Default);

extern void AndroidThunkCpp_InitHMDs();
extern void AndroidThunkCpp_ShowConsoleWindow();
extern bool AndroidThunkCpp_VirtualInputIgnoreClick(int, int);
extern bool AndroidThunkCpp_IsVirtualKeyboardShown();
extern bool AndroidThunkCpp_IsWebViewShown();
extern bool AndroidThunkCpp_HasExtraIgnoreView();
extern void AndroidThunkCpp_RestartApplication(const FString& IntentString);
extern FString AndroidThunkCpp_GetIntentExtrasString(const FString& Key);

// Base path for file accesses
extern FString GFilePathBase;

/** The global EngineLoop instance */
FEngineLoop	GEngineLoop;

#if USE_ANDROID_STANDALONE
FString GAndroidCommandLine;
#endif

extern void* GAndroidWindowOverride;
extern int GSurfaceViewWidth;
extern int GSurfaceViewHeight;

static bool bDidCompleteEngineInit = false;

#if !UE_BUILD_SHIPPING
bool GShowConsoleWindowNextTick = false;
extern bool GDebugConsoleOpen; // from AndroidPlatformMisc.cpp
#endif

static void AndroidProcessEvents(struct android_app* state);

//Event thread stuff
static void* AndroidEventThreadWorker(void* param);

// Name of the UE commandline append setprop
static constexpr char UECommandLineSetprop[] = "debug.ue.commandline";

//Android event callback functions
static int32_t HandleInputCB(struct android_app* app, AInputEvent* event); //Touch and key input events
static void OnAppCommandCB(struct android_app* app, int32_t cmd); //Lifetime events

static bool TryIgnoreClick(AInputEvent* event, size_t actionPointer);

bool GAllowJavaBackButtonEvent = false;
bool GHasInterruptionRequest = false;
bool GIsInterrupted = false;

// Set 'SustainedPerformanceMode' via cvar sink.
static TAutoConsoleVariable<int32> CVarEnableSustainedPerformanceMode(
	TEXT("Android.EnableSustainedPerformanceMode"),
	0,
	TEXT("Enable sustained performance mode, if supported. (API >= 24 req. not supported by all devices.)\n")
	TEXT("  0: Disabled (default)\n")
	TEXT("  1: Enabled"),
	ECVF_Default);

extern void AndroidThunkCpp_SetSustainedPerformanceMode(bool);
static void SetSustainedPerformanceMode()
{
	static bool bSustainedPerformanceMode = false;
	bool bIncomingSustainedPerformanceMode = CVarEnableSustainedPerformanceMode.GetValueOnAnyThread() != 0;
	if (bSustainedPerformanceMode != bIncomingSustainedPerformanceMode)
	{
		bSustainedPerformanceMode = bIncomingSustainedPerformanceMode;
		UE_LOGF(LogAndroid, Log, "Setting sustained performance mode: %d", (int32)bSustainedPerformanceMode);
		AndroidThunkCpp_SetSustainedPerformanceMode(bSustainedPerformanceMode);
	}
}
FAutoConsoleVariableSink CVarEnableSustainedPerformanceModeSink(FConsoleCommandDelegate::CreateStatic(&SetSustainedPerformanceMode));

// Event for coordinating pausing of the main and event handling threads to prevent background spinning
static FEvent* EventHandlerEvent = NULL;

// Wait for Java onCreate to complete before resume main init
static volatile bool GResumeMainInit = false;
volatile bool GEventHandlerInitialized = false;

static void IssueConsoleCommand(FString Command)
{
	static TArray<FString> GPendingConsoleCommands;
	if (GEngine != NULL)
	{
		// Run on game thread to avoid race condition with DeferredCommands
		AsyncTask(ENamedThreads::GameThread, [Command]()
			{
				if (GPendingConsoleCommands.Num() > 0)
				{
					STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("IssueConsoleCommand: Pending count= %d"), GPendingConsoleCommands.Num());
					for (FString& pendingCommand : GPendingConsoleCommands)
					{
						STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("IssuePendingConsoleCommand: %s"), *pendingCommand);
						GEngine->DeferredCommands.Add(pendingCommand);
					}
					GPendingConsoleCommands.Empty();
				}
				STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("IssueConsoleCommand: %s"), *Command);
				GEngine->DeferredCommands.Add(Command);
			});
	}
	// to avoid out of order console command issues that were overwritten in configs, for non standalone we do not store pending.
#if USE_ANDROID_STANDALONE
	else
	{
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Ignoring console command (too early, Queue for later): %s"), *Command);
		GPendingConsoleCommands.Add(MoveTemp(Command));
	}
#endif
}

// The event thread locks this whenever the window unavailable during early init, pause and resume.
FCriticalSection GAndroidWindowLock;
static bool bAppIsActive_EventThread = false;
static bool bHasFocus = false;
static bool bHasWindow = false;
static bool bIsResumed = false;
static bool bShouldRestartFromInterrupt = false;
static bool bIgnorePauseOnDownloaderStart = false;
bool bReadyToProcessEvents = false;

pthread_t G_AndroidEventThread;
struct android_app* GNativeAndroidApp = NULL;
static uint32 EventThreadID = 0;


static void SuspendApp_EventThread();
static void ActivateApp_EventThread();
#if USE_ANDROID_STANDALONE && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define DEVELOPER_LOG_COMMANDCB_CASE(a) STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("--TRACE-- OnAppCommandCB - Case [%s] : bHasFocus = %d, bHasWindow = %d, bIsResumed = %d, bShouldRestartFromInterrupt = %d, bReadyToProcessEvents=%d, bAppIsActive_EventThread=%d"), TEXT(#a), bHasFocus, bHasWindow, bIsResumed, bShouldRestartFromInterrupt, bReadyToProcessEvents, bAppIsActive_EventThread)
#else
#define DEVELOPER_LOG_COMMANDCB_CASE(a) 
#endif

void FPlatformMisc::UnlockAndroidWindow()
{
	check(IsInGameThread());
	check(FTaskGraphInterface::IsRunning());
	
	bReadyToProcessEvents = true;
	if(!FAndroidMisc::UseNewWindowBehavior())
	{
		UE_LOGF(LogAndroid, Log, "Unlocking android HW window during preinit. bAppIsActive_EventThread=%d", bAppIsActive_EventThread);
		GAndroidWindowLock.Unlock();
	}
}

void JNICALL UE::Jni::FGameActivity::nativeResumeMainInit(JNIEnv* jenv, jobject thiz)
{
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("nativeResumeMainInit() called, prev GResumeMainInit=%d, GEventHandlerInitialized=%d"), GResumeMainInit, GEventHandlerInitialized);

	GResumeMainInit = true;

	// now wait for event handler to be set up before returning
	while (!GEventHandlerInitialized)
	{
		FPlatformProcess::Sleep(0.01f);
		FPlatformMisc::MemoryBarrier();
	}
}

static volatile bool GHMDsInitialized = false;
static TArray<IHeadMountedDisplayModule*> GHMDImplementations;
void InitHMDs()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("nohmd")) || FParse::Param(FCommandLine::Get(), TEXT("emulatestereo")))
	{
		return;
	}

	// Get a list of plugins that implement this feature
	GHMDImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IHeadMountedDisplayModule>(IHeadMountedDisplayModule::GetModularFeatureName());
	if (GHMDImplementations.IsEmpty())
	{
		return;
	}

	AndroidThunkCpp_InitHMDs();

	while (!GHMDsInitialized)
	{
		FPlatformProcess::Sleep(0.01f);
		FPlatformMisc::MemoryBarrier();
	}
}

extern AAssetManager* AndroidThunkCpp_GetAssetManager();

/**
 * Shuts down the engine
 */
void EngineExit(void)
{
	// Make sure this is set
	RequestEngineExit(TEXT("EngineExit() was called"));
	GEngineLoop.Exit();
	UE_LOGF(LogAndroid, Log, "Exiting is over");
	FPlatformMisc::RequestExit(1);
}

static void InitCommandLine()
{
	static constexpr int32 CMD_LINE_MAX = 16384;
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("LaunchAndroid::InitCommandLine called!"));

	// initialize the command line to an empty string
	FCommandLine::Set(TEXT(""));

#if !UE_BUILD_SHIPPING
	FString CmdLine = AndroidThunkCpp_GetIntentExtrasString(TEXT("cmdline"));
	if (!CmdLine.IsEmpty())
	{
		CmdLine.TrimEndInline();
		FCommandLine::Append(*CmdLine);

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("adb am start command line override: %s"), FCommandLine::Get());
	}
	else
#endif
	{
		AAssetManager* AssetMgr = AndroidThunkCpp_GetAssetManager();
		AAsset* asset = AAssetManager_open(AssetMgr, "UECommandLine.txt", AASSET_MODE_BUFFER);
		if (nullptr != asset)
		{
			const void* FileContents = AAsset_getBuffer(asset);
			int32 CommandLineLength = int32(FMath::Min(AAsset_getLength(asset), off_t(CMD_LINE_MAX - 1)));

			UTF8CHAR CommandLine[CMD_LINE_MAX];
			memcpy(CommandLine, FileContents, CommandLineLength);

			AAsset_close(asset);

			// chop off trailing spaces
			CommandLineLength = FUtf8StringView{CommandLine, CommandLineLength}.TrimEnd().Len();
			CommandLine[CommandLineLength] = '\0';

			FCommandLine::Append(StringCast<TCHAR>(CommandLine, CommandLineLength + 1).Get());
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("APK Commandline: %s"), FCommandLine::Get());
		}

#if !UE_BUILD_SHIPPING && USE_ANDROID_STANDALONE
		UE_SET_LOG_VERBOSITY(LogAndroid, Log);
		if (!GAndroidCommandLine.IsEmpty())
		{
			FCommandLine::Set(TEXT(""));
			FCommandLine::Append(*GAndroidCommandLine);
			STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Override Commandline: %s"), FCommandLine::Get());
		}
#endif

		// read in the command line text file from the sdcard if it exists
		FString CommandLineFilePath = GFilePathBase + FString("/UnrealGame/") + (!FApp::IsProjectNameEmpty() ? FApp::GetProjectName() : FPlatformProcess::ExecutableName()) + FString("/UECommandLine.txt");
		FILE* CommandLineFile = fopen(TCHAR_TO_UTF8(*CommandLineFilePath), "r");
		if (CommandLineFile == NULL)
		{
			// if that failed, try the lowercase version
			CommandLineFilePath = CommandLineFilePath.Replace(TEXT("UECommandLine.txt"), TEXT("uecommandline.txt"));
			CommandLineFile = fopen(TCHAR_TO_UTF8(*CommandLineFilePath), "r");
		}

		if (CommandLineFile)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Using override commandline file: %s"), *CommandLineFilePath);

			// initialize the command line to an empty string
			FCommandLine::Set(TEXT(""));

			UTF8CHAR CommandLine[CMD_LINE_MAX];
			if (fgets((char*)CommandLine, CMD_LINE_MAX, CommandLineFile))
			{
				// chop off trailing spaces
				int32 CommandLineLength = FUtf8StringView{CommandLine}.TrimEnd().Len();
				CommandLine[CommandLineLength] = '\0';

				FCommandLine::Append(StringCast<TCHAR>(CommandLine, CommandLineLength + 1).Get());
			}

			fclose(CommandLineFile);

			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Override Commandline: %s"), FCommandLine::Get());
		}
	}

#if !UE_BUILD_SHIPPING
	if (const FString* ConfigRulesCmdLineAppend = FAndroidMisc::GetConfigRulesVariable(TEXT("cmdline")))
	{
		FCommandLine::Append(**ConfigRulesCmdLineAppend);
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("ConfigRules appended: %s"), **ConfigRulesCmdLineAppend);
	}

	char CommandLineSetpropAppend[CMD_LINE_MAX];
	if (__system_property_get(UECommandLineSetprop, CommandLineSetpropAppend) > 0)
	{
		FCommandLine::Append(UTF8_TO_TCHAR(" "));
		FCommandLine::Append(UTF8_TO_TCHAR(CommandLineSetpropAppend));
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("UE setprop appended: %s"), UTF8_TO_TCHAR(CommandLineSetpropAppend));
	}
#endif

#ifdef UE_ANDROID_COMMAND_LINE_OVERRIDE
	FCommandLine::Set(TEXT(UE_ANDROID_COMMAND_LINE_OVERRIDE));
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("UE_ANDROID_COMMAND_LINE_OVERRIDE: %s"), TEXT(UE_ANDROID_COMMAND_LINE_OVERRIDE));
#endif

#ifdef UE_ANDROID_COMMAND_LINE_APPEND
	FCommandLine::Append(TEXT(UE_ANDROID_COMMAND_LINE_APPEND));
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("UE_ANDROID_COMMAND_LINE_APPEND: %s"), TEXT(UE_ANDROID_COMMAND_LINE_APPEND));
#endif

}

extern void AndroidThunkCpp_DismissSplashScreen();

//Called in main thread for native window resizing
static void OnNativeWindowResized(ANativeActivity* activity, ANativeWindow* window)
{
	static int8_t cmd = APP_CMD_WINDOW_RESIZED;
	struct android_app* app = (struct android_app *)activity->instance;
	write(app->msgwrite, &cmd, sizeof(cmd));
}

static void ApplyAndroidCompatConfigRules()
{
	TArray<FString> AndroidCompatCVars;
	if (GConfig->GetArray(TEXT("AndroidCompatCVars"), TEXT("CVars"), AndroidCompatCVars, GEngineIni))
	{
		TSet<FString> AllowedCompatCVars(AndroidCompatCVars);
		for (const TTuple<FString, FString>& Pair : FPlatformMisc::GetConfigRuleVars())
		{
			const FString& Key = Pair.Key;
			const FString& Value = Pair.Value;
			static const TCHAR AndroidCompat[] = TEXT("AndroidCompat.");
			if (Key.StartsWith(AndroidCompat))
			{
				FString CVarName = Key.Mid(UE_ARRAY_COUNT(AndroidCompat)-1);
				if (AllowedCompatCVars.Contains(CVarName))
				{
					auto* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
					if (CVar)
					{
						// set with HF priority
						// configrules are therefore higher priority than deviceprofiles. (e.g. -dpcvars)
						CVar->Set(*Value, ECVF_SetByHotfix);
						UE_LOGF(LogAndroid, Log, "Compat Setting %ls = %ls", *CVarName, *Value);
					}
				}
			}
		}
	}
}

static bool ShouldUseLoggingThread()
{
	bool bAllowLoggingThread = false;
	GConfig->GetBool(FPlatformProperties::GetRuntimeSettingsClassName(), TEXT("bAllowLoggingThread"), bAllowLoggingThread, GEngineIni);

	const FString* ConfigRulesOverride = FPlatformMisc::GetConfigRulesVariable(TEXT("AllowLoggingThread"));
	if (ConfigRulesOverride)
	{
		bAllowLoggingThread = ConfigRulesOverride->Equals("true", ESearchCase::IgnoreCase);
	}

	return bAllowLoggingThread;
}

//Main function called from the android entry point
int32 AndroidMain(struct android_app* state);

// The function is used only for ASIS
void* AndroidMain(void* param)
{
	struct android_app* state = (struct android_app*)param;
	FTaskTagScope Scope(ETaskTag::EGameThread);

	if (GGameThreadId == 0)
	{
		GGameThreadId = FPlatformTLS::GetCurrentThreadId();
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidMain set current GGameThreadId=%d"), GGameThreadId);
	}

	AndroidMain(state);
	return nullptr;
}


int32 AndroidMain(struct android_app* state)
{
	BootTimingPoint("AndroidMain");

	FPlatformMisc::LowLevelOutputDebugString(TEXT("Entered AndroidMain()\n"));

	// Force the first call to GetJavaEnv() to happen on the game thread, allowing subsequent calls to occur on any thread
	FAndroidApplication::GetJavaEnv();
#if !USE_ANDROID_STANDALONE
	// Set window format to 8888
	ANativeActivity_setWindowFormat(state->activity, WINDOW_FORMAT_RGBA_8888);
#endif

#if !UE_BUILD_SHIPPING && USE_ANDROID_STANDALONE
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("ENABLE LogAndroid verbosity!\n"));
	UE_SET_LOG_VERBOSITY(LogAndroid, Log);
#endif

	// adjust the file descriptor limits to allow as many open files as possible
	rlimit cur_fd_limit;
	{
		int result = getrlimit(RLIMIT_NOFILE, & cur_fd_limit);
		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("(%d) Current fd limits: soft = %lld, hard = %lld"), result, cur_fd_limit.rlim_cur, cur_fd_limit.rlim_max);
	}
	{
		rlimit new_limit = cur_fd_limit;
		new_limit.rlim_cur = cur_fd_limit.rlim_max;
		new_limit.rlim_max = cur_fd_limit.rlim_max;
		int result = setrlimit(RLIMIT_NOFILE, &new_limit);
		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("(%d) Setting fd limits: soft = %lld, hard = %lld"), result, new_limit.rlim_cur, new_limit.rlim_max);
	}
	{
		int result = getrlimit(RLIMIT_NOFILE, & cur_fd_limit);
		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("(%d) Current fd limits: soft = %lld, hard = %lld"), result, cur_fd_limit.rlim_cur, cur_fd_limit.rlim_max);
	}

	// setup joystick support
	// r19 is the first NDK to include AMotionEvent_getAxisValue in the headers
	// However, it has existed in the so since Honeycomb, query for the symbol
	// to determine whether to try controller support
	{
		void* Lib = dlopen("libandroid.so",0);
		if (Lib != NULL)
		{
			GetAxes = (GetAxesType)dlsym(Lib, "AMotionEvent_getAxisValue");
		}

		if (GetAxes != NULL)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Controller interface supported\n"));
		}
		else
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Controller interface UNsupported\n"));
		}
	}

	// setup key filtering
	static const uint32 MAX_KEY_MAPPINGS(256);
	uint32 KeyCodes[MAX_KEY_MAPPINGS];
	uint32 NumKeyCodes = FPlatformInput::GetKeyMap(KeyCodes, nullptr, MAX_KEY_MAPPINGS);

	for (int i = 0; i < NumKeyCodes; ++i)
	{
		MappedKeyCodes.Add(KeyCodes[i]);
	}

	const int AlwaysAllowedKeyCodesCount = sizeof(AlwaysAllowedKeyCodesList) / sizeof(uint32);
	for (int i = 0; i < AlwaysAllowedKeyCodesCount; ++i)
	{
		AlwaysAllowedKeyCodes.Add(AlwaysAllowedKeyCodesList[i]);
	}

	const int ValidGamepadKeyCodeCount = sizeof(ValidGamepadKeyCodesList)/sizeof(uint32);
	for (int i = 0; i < ValidGamepadKeyCodeCount; ++i)
	{
		ValidGamepadKeyCodes.Add(ValidGamepadKeyCodesList[i]);
	}

	FAndroidPlatformStackWalk::InitStackWalking();
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT(" wait for java activity onCreate to finish, GResumeMainInit=%d"), GResumeMainInit);

	// wait for java activity onCreate to finish
	{
		SCOPED_BOOT_TIMING("Wait for GResumeMainInit");
		while (!GResumeMainInit)
		{
			FPlatformProcess::Sleep(0.01f);
			FPlatformMisc::MemoryBarrier();
		}
	}


	// read the command line file
	InitCommandLine();
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Final commandline: %s (len %i)\n"), FCommandLine::Get(), FCString::Strlen(FCommandLine::Get()));
	const TCHAR* CmdLine = FCommandLine::Get();

#if !(UE_BUILD_SHIPPING)
	// If "-waitforattach" or "-WaitForDebugger" was specified, halt startup and wait for a debugger to attach before continuing
	if (FParse::Param(FCommandLine::Get(), TEXT("waitforattach")) || FParse::Param(FCommandLine::Get(), TEXT("WaitForDebugger")))
	{
		FPlatformMisc::LowLevelOutputDebugString(TEXT("Waiting for debugger to attach...\n"));
		while (!FPlatformMisc::IsDebuggerPresent());
		UE_DEBUG_BREAK();
	}
#endif

	EventHandlerEvent = FPlatformProcess::GetSynchEventFromPool(false);
	FPlatformMisc::LowLevelOutputDebugString(TEXT("Created sync event\n"));
	FAppEventManager::GetInstance()->SetEventHandlerEvent(EventHandlerEvent);

	// ready for onCreate to complete
	GEventHandlerInitialized = true;

	// Initialize file system access (i.e. mount OBBs, etc.).
	// We need to do this really early for Android so that files in the
	// OBBs and APK are found.
	// Have to use a special initialize if using the PersistentStorageManager
	IPlatformFile::GetPlatformPhysical().Initialize(nullptr, FCommandLine::Get());

	if(!FAndroidMisc::UseNewWindowBehavior())
	{
		SCOPED_BOOT_TIMING("Wait for GAndroidWindowLock.Lock()");
		// wait for a valid window
		// Lock GAndroidWindowLock to ensure no window destroy shenanigans occur between early phase of preinit and UnlockAndroidWindow
		// Note: this is unlocked after Android's PlatformCreateDynamicRHI when the RHI is then able to process window changes.
		// We don't wait for all of preinit to complete as PreLoadScreens will need to process events during preinit.

		UE_LOGF(LogAndroid, Log, "PreInit android HW window lock. bAppIsActive_EventThread=%d", bAppIsActive_EventThread);
		GAndroidWindowLock.Lock();
	}
	else
	{
		UE_LOGF(LogAndroid, Log, "PreInit android using new window behavior. bAppIsActive_EventThread=%d", bAppIsActive_EventThread);
	}

	FPlatformMisc::LowLevelOutputDebugString(TEXT("After GAndroidWindowLock in AndroidMain"));

	FDelegateHandle ConfigReadyHandle = FCoreDelegates::TSConfigReadyForUse().AddStatic(&ApplyAndroidCompatConfigRules);

	// initialize the engine
	int32 PreInitResult = GEngineLoop.PreInit(0, NULL, FCommandLine::Get());

	FCoreDelegates::TSConfigReadyForUse().Remove(ConfigReadyHandle);

	if (PreInitResult != 0)
	{
		checkf(false, TEXT("Engine Preinit Failed"));
		return PreInitResult;
	}

	// register callback for native window resize
	if (GAndroidEnableNativeResizeEvent)
	{
		state->activity->callbacks->onNativeWindowResized = OnNativeWindowResized;
	}

	// initialize HMDs
	{
		SCOPED_BOOT_TIMING("InitHMDs");
		InitHMDs();
	}

	UE_LOGF(LogAndroid, Display, "Passed PreInit()");

	if (!ShouldUseLoggingThread())
	{
		GLog->SetCurrentThreadAsPrimaryThread();
	}

	int32 ErrorLevel = PreInitResult;

	FAppEventManager::GetInstance()->SetEmptyQueueHandlerEvent(FPlatformProcess::GetSynchEventFromPool(false));

	ErrorLevel = GEngineLoop.Init();

	bDidCompleteEngineInit = true;

	UE_LOGF(LogAndroid, Log, "Passed GEngineLoop.Init()");
	double EngineInitializationTime = FPlatformTime::Seconds() - GStartTime;
	UE_LOGF(LogLoad, Log, "(Engine Initialization) Total time: %.2f seconds", EngineInitializationTime);
	ACCUM_LOADTIME(TEXT("EngineInitialization"), EngineInitializationTime);

	AndroidThunkCpp_DismissSplashScreen();

#if !UE_BUILD_SHIPPING
	if (FParse::Param(FCommandLine::Get(), TEXT("Messaging")))
	{
		// initialize messaging subsystem
		FModuleManager::LoadModuleChecked<IMessagingModule>("Messaging");
		TSharedPtr<ISessionService> SessionService = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices").GetSessionService();
		SessionService->Start();

		// Initialize functional testing
		FModuleManager::Get().LoadModule("FunctionalTesting");
	}
#endif

	FAndroidStats::Init();

	BootTimingPoint("Tick loop starting");
	DumpBootTiming();

	// tick until done
	while (!IsEngineExitRequested())
	{
		FAndroidStats::UpdateAndroidStats();

		FAppEventManager::GetInstance()->Tick();
		if (!FAppEventManager::GetInstance()->IsGamePaused()
#if USE_ANDROID_STANDALONE
			//&& FAppEventManager::GetInstance()->IsGameInFocus()
#endif
			)
		{
			GEngineLoop.Tick();
		}
		else
		{
			// use less CPU when paused
			FPlatformProcess::Sleep(0.10f);

//	TODO:we could continue to tick the engine loop as the new window behavior no longer blocks the RT .
// 			if (FAndroidMisc::UseNewWindowBehavior())
// 			{
// 				GEngineLoop.Tick();
// 			}
		}

#if !UE_BUILD_SHIPPING
		// show console window on next game tick
		if (GShowConsoleWindowNextTick)
		{
			GShowConsoleWindowNextTick = false;
			AndroidThunkCpp_ShowConsoleWindow();
		}
#endif
	}
	TRACE_BOOKMARK(TEXT("Tick loop end"));
	
	FAppEventManager::GetInstance()->TriggerEmptyQueue();

	UE_LOGF(LogAndroid, Log, "Exiting");

	if (GAndroidTerminateOnExit && !FParse::Param(FCommandLine::Get(), TEXT("ProperShutdown")))
	{
		UE_LOGF(LogAndroid, Log, "Terminating process (skip shutdown)");
		FPlatformMisc::RequestExit(true, TEXT("AndroidMain"));
		return 0;
	}

	// exit out!
	GEngineLoop.Exit();

	UE_LOGF(LogAndroid, Log, "Exiting is over");

	FPlatformMisc::RequestExit(true, TEXT("AndroidMain"));

	return 0;
}

struct AChoreographer;
struct FChoreographer
{
	typedef void(*AChoreographer_frameCallback)(long frameTimeNanos, void* data);
	typedef AChoreographer* (*func_AChoreographer_getInstance)();
	typedef void(*func_AChoreographer_postFrameCallback)(
		AChoreographer* choreographer, AChoreographer_frameCallback callback,
		void* data);
	typedef void(*func_AChoreographer_postFrameCallbackDelayed)(
		AChoreographer* choreographer, AChoreographer_frameCallback callback,
		void* data, long delayMillis);

	func_AChoreographer_getInstance AChoreographer_getInstance_ = nullptr;
	func_AChoreographer_postFrameCallback AChoreographer_postFrameCallback_ = nullptr;
	func_AChoreographer_postFrameCallbackDelayed AChoreographer_postFrameCallbackDelayed_ = nullptr;

	FCriticalSection ChoreographerSetupLock;

	TFunction<int64(int64)> Callback;

	void SetupChoreographer()
	{
		FScopeLock Lock(&ChoreographerSetupLock);
		//check(!AChoreographer_getInstance_);
		if (!AChoreographer_getInstance_)
		{
			void* lib = dlopen("libandroid.so", RTLD_NOW | RTLD_LOCAL);
			if (lib != nullptr)
			{
				// Retrieve function pointers from shared object.
				AChoreographer_getInstance_ =
					reinterpret_cast<func_AChoreographer_getInstance>(
						dlsym(lib, "AChoreographer_getInstance"));
				AChoreographer_postFrameCallback_ =
					reinterpret_cast<func_AChoreographer_postFrameCallback>(
						dlsym(lib, "AChoreographer_postFrameCallback"));
				AChoreographer_postFrameCallbackDelayed_ =
					reinterpret_cast<func_AChoreographer_postFrameCallbackDelayed>(
						dlsym(lib, "AChoreographer_postFrameCallbackDelayed"));
			}

			if (!AChoreographer_getInstance_ || !AChoreographer_postFrameCallback_ || !AChoreographer_postFrameCallbackDelayed_)
			{
				UE_LOGF(LogAndroid, Warning, "Failed to set up Choreographer");
				AChoreographer_getInstance_ = nullptr;
				AChoreographer_postFrameCallback_ = nullptr;
				AChoreographer_postFrameCallbackDelayed_ = nullptr;
			}
			else
			{
				SetCallback(0);
				UE_LOGF(LogAndroid, Display, "Choreographer set up.");
			}
		}
	}
	void SetupCallback(TFunction<int64(int64)> InCallback)
	{
		check(IsAvailable());
		FScopeLock Lock(&ChoreographerSetupLock);
		Callback = InCallback;
	}

	void SetCallback(int64 Delay);
	void DoCallback(long frameTimeNanos)
	{
		//static long LastFrameTimeNanos = 0;
		//UE_LOGF(LogAndroid, Warning, "Choreographer %lld   delta %lld", frameTimeNanos, frameTimeNanos - LastFrameTimeNanos);
		//LastFrameTimeNanos = frameTimeNanos;
		int64 NextDelay = -1;
		{
			FScopeLock Lock(&ChoreographerSetupLock);
			if (Callback)
			{
				NextDelay = Callback(frameTimeNanos);
			}
		}
		SetCallback((NextDelay >= 0) ? NextDelay : 0);
	}
	bool IsAvailable()
	{
		return !!AChoreographer_getInstance_;
	}
};
FChoreographer TheChoreographer;

bool ChoreographerIsAvailable()
{
	return TheChoreographer.IsAvailable();
}

void StartChoreographer(TFunction<int64(int64)> Callback)
{
	check(ChoreographerIsAvailable());
	TheChoreographer.SetupCallback(Callback);
}


static void choreographer_callback(long frameTimeNanos, void* data)
{
	TheChoreographer.DoCallback(frameTimeNanos);
}

void FChoreographer::SetCallback(int64 Delay)
{
	check(IsAvailable());
	check(Delay >= 0);
	AChoreographer* choreographer = AChoreographer_getInstance_();
	UE_CLOGF(!choreographer, LogAndroid, Fatal, "Choreographer was null (wrong thread?).");
	AChoreographer_postFrameCallbackDelayed_(choreographer, choreographer_callback, nullptr, Delay / 1000000);
}

bool IsInAndroidEventThread()
{
	// Note: leave the commented out line for debug purposes.
	//STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("IsInAndroidEventThread(), GGameThreadId=%d, EventThreadID=%d, FPlatformTLS::GetCurrentThreadId()=%d"), GGameThreadId, EventThreadID, FPlatformTLS::GetCurrentThreadId());

	check(EventThreadID != 0);
	return EventThreadID == FPlatformTLS::GetCurrentThreadId();
}

static void* AndroidEventThreadWorker( void* param )
{
	FTaskTagScope::SwapTag(ETaskTag::EEventThread);
	pthread_setname_np(pthread_self(), "EventWorker");
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidEventThreadWorker(begin), GGameThreadId=%d, EventThreadID=%d, FPlatformTLS::GetCurrentThreadId()=%d"), GGameThreadId, EventThreadID, FPlatformTLS::GetCurrentThreadId());

	EventThreadID = FPlatformTLS::GetCurrentThreadId();
	FAndroidMisc::RegisterThreadName("EventWorker", EventThreadID);

	struct android_app* state = (struct android_app*)param;

	FPlatformProcess::SetThreadAffinityMask(FPlatformAffinity::GetMainGameMask());

	FPlatformMisc::LowLevelOutputDebugString(TEXT("Entering event processing thread engine entry point"));
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidEventThreadWorker(started), EventThreadID=%d"), EventThreadID);

	ALooper* looper = ALooper_prepare(ALOOPER_PREPARE_ALLOW_NON_CALLBACKS);
	ALooper_addFd(looper, state->msgread, LOOPER_ID_MAIN, ALOOPER_EVENT_INPUT, NULL,
		&state->cmdPollSource);
	state->looper = looper;

	FPlatformMisc::LowLevelOutputDebugString(TEXT("Prepared looper for event thread"));

	//Assign the callbacks
	state->onAppCmd = OnAppCommandCB;
	state->onInputEvent = HandleInputCB;

	FPlatformMisc::LowLevelOutputDebugString(TEXT("Passed callback initialization"));
	FPlatformMisc::LowLevelOutputDebugString(TEXT("Passed sensor initialization"));

	TheChoreographer.SetupChoreographer();

	if(!FAndroidMisc::UseNewWindowBehavior())
	{
		// window is initially invalid/locked.
		UE_LOGF(LogAndroid, Log, "AndroidEventThreadWorker, Initial HW window lock.");
		GAndroidWindowLock.Lock();
	}

	DEVELOPER_LOG_COMMANDCB_CASE(AndroidEventThreadWorker_BeforeWhile);


	//continue to process events until the engine is shutting down
	while (!IsEngineExitRequested())
	{
		AndroidProcessEvents(state);
	}
	DEVELOPER_LOG_COMMANDCB_CASE(AndroidEventThreadWorker_AfterWhile);

	if (!FAndroidMisc::UseNewWindowBehavior())
	{
		GAndroidWindowLock.Unlock();
	}

	UE_LOGF(LogAndroid, Log, "AndroidEventThreadWorker->Exiting");
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Exiting AndroidEventThreadWorker"));

	return NULL;
}

//Called from the separate event processing thread
static void AndroidProcessEvents(struct android_app* State)
{
	struct android_poll_source* Source = nullptr;
	int32 Result = ALooper_pollOnce(-1, nullptr, nullptr, (void**)&Source);

	if (Result != ALOOPER_POLL_ERROR && Source != nullptr)
	{
		Source->process(State, Source);
	}
}

void android_main(struct android_app* state)
{
	FTaskTagScope Scope(ETaskTag::EGameThread);

	GGameThreadId = FPlatformTLS::GetCurrentThreadId();

	BootTimingPoint("android_main");
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Entering native app glue main function, GGameThreadId=%d"), GGameThreadId);
	
	GNativeAndroidApp = state;
	check(GNativeAndroidApp);
#if USE_ANDROID_STANDALONE
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("android_main : USE_ANDROID_STANDALONE is defined and enabled"));
#endif

	pthread_attr_t otherAttr; 
	pthread_attr_init(&otherAttr);
	pthread_attr_setdetachstate(&otherAttr, PTHREAD_CREATE_DETACHED);
	pthread_create(&G_AndroidEventThread, &otherAttr, AndroidEventThreadWorker, state);

	FPlatformMisc::LowLevelOutputDebugString(TEXT("Created event thread"));

	// Make sure glue isn't stripped. (not needed in ndk-15)
#if PLATFORM_ANDROID_NDK_VERSION < 150000
	app_dummy();
#endif

	//@todo android: replace with native activity, main loop off of UI thread, etc.
	AndroidMain(state);
}

void* android_main(void* param)
{
	struct android_app* state = (struct android_app*)param;
	android_main(state);
	return nullptr;
}

extern bool GAndroidGPUInfoReady;

static bool TryIgnoreClick(AInputEvent* event, size_t actionPointer)
{
	int pointerId = AMotionEvent_getPointerId(event, actionPointer);
	int32 x = AMotionEvent_getX(event, actionPointer);
	int32 y = AMotionEvent_getY(event, actionPointer);

	//ignore key down events click was within bounds
	if (AndroidThunkCpp_VirtualInputIgnoreClick(x, y))
	{
		return true;
	}
	return false;
}

//Called from the event process thread
static int32_t HandleInputCB(struct android_app* app, AInputEvent* event)
{
//	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("INPUT - type: %x, action: %x, source: %x, keycode: %x, buttons: %x"), AInputEvent_getType(event), 
//		AMotionEvent_getAction(event), AInputEvent_getSource(event), AKeyEvent_getKeyCode(event), AMotionEvent_getButtonState(event));
	check(IsInAndroidEventThread());

	int32 EventType = AInputEvent_getType(event);
	int32 EventSource = AInputEvent_getSource(event);

	if ((EventSource & AINPUT_SOURCE_MOUSE) == AINPUT_SOURCE_MOUSE)
	{
		static int32 previousButtonState = 0;

		const int32 device = AInputEvent_getDeviceId(event);
		const int32 action = AMotionEvent_getAction(event);
		const int32 actionType = action & AMOTION_EVENT_ACTION_MASK;
		int32 buttonState = AMotionEvent_getButtonState(event);

		if (!GAndroidEnableMouse)
		{
			if (actionType == AMOTION_EVENT_ACTION_DOWN || actionType == AMOTION_EVENT_ACTION_UP)
			{
				const bool bDown = (actionType == AMOTION_EVENT_ACTION_DOWN);
				if (!bDown)
				{
					buttonState = previousButtonState;
				}
				if (buttonState & AMOTION_EVENT_BUTTON_PRIMARY)
				{
					const int32 ReplacementKeyEvent = FAndroidInputInterface::GetAlternateKeyEventForMouse(device, 0);
					if (ReplacementKeyEvent != 0)
					{
						FAndroidInputInterface::JoystickButtonEvent(device, ReplacementKeyEvent, bDown);
					}
				}
				if (buttonState & AMOTION_EVENT_BUTTON_SECONDARY)
				{
					const int32 ReplacementKeyEvent = FAndroidInputInterface::GetAlternateKeyEventForMouse(device, 1);
					if (ReplacementKeyEvent != 0)
					{
						FAndroidInputInterface::JoystickButtonEvent(device, ReplacementKeyEvent, bDown);
					}
				}
				if (buttonState & AMOTION_EVENT_BUTTON_TERTIARY)
				{
					const int32 ReplacementKeyEvent = FAndroidInputInterface::GetAlternateKeyEventForMouse(device, 2);
					if (ReplacementKeyEvent != 0)
					{
						FAndroidInputInterface::JoystickButtonEvent(device, ReplacementKeyEvent, bDown);
					}
				}
				previousButtonState = buttonState;
			}
			return 1;
		}

//		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("-- EVENT: %d, device: %d, action: %x, actionType: %x, buttonState: %x"), EventType, device, action, actionType, buttonState);

		if (actionType == AMOTION_EVENT_ACTION_DOWN || actionType == AMOTION_EVENT_ACTION_UP)
		{
			const bool bDown = (actionType == AMOTION_EVENT_ACTION_DOWN);
			if (!bDown)
			{
				buttonState = previousButtonState;
			}
			if (buttonState & AMOTION_EVENT_BUTTON_PRIMARY)
			{
//				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mouse button 0: %d"), bDown ? 1 : 0);
				FAndroidInputInterface::MouseButtonEvent(device, 0, bDown);
			}
			if (buttonState & AMOTION_EVENT_BUTTON_SECONDARY)
			{
//				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mouse button 1: %d"), bDown ? 1 : 0);
				FAndroidInputInterface::MouseButtonEvent(device, 1, bDown);
			}
			if (buttonState & AMOTION_EVENT_BUTTON_TERTIARY)
			{
//				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mouse button 2: %d"), bDown ? 1 : 0);
				FAndroidInputInterface::MouseButtonEvent(device, 2, bDown);
			}
			previousButtonState = buttonState;
			return 1;
		}

		if (actionType == AMOTION_EVENT_ACTION_SCROLL)
		{
			if (GetAxes)
			{
				float WheelDelta = GetAxes(event, AMOTION_EVENT_AXIS_VSCROLL, 0);

				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mouse scroll: %f"), WheelDelta);
				FAndroidInputInterface::MouseWheelEvent(device, WheelDelta);
			}
			return 1;
		}

		if (GetAxes && (actionType == AMOTION_EVENT_ACTION_MOVE || actionType == AMOTION_EVENT_ACTION_HOVER_MOVE))
		{
			float XAbsolute = GetAxes(event, AMOTION_EVENT_AXIS_X, 0);
			float YAbsolute = GetAxes(event, AMOTION_EVENT_AXIS_Y, 0);
			float XRelative = GetAxes(event, AMOTION_EVENT_AXIS_RELATIVE_X, 0);
			float YRelative = GetAxes(event, AMOTION_EVENT_AXIS_RELATIVE_Y, 0);

			if (XRelative != 0.0f || YRelative != 0.0f)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Mouse absolute: (%f, %f), relative (%f, %f)"), XAbsolute, YAbsolute, XRelative, YRelative);
				FAndroidInputInterface::MouseMoveEvent(device, XAbsolute, YAbsolute, XRelative, YRelative);
			}
		}

		return 1;
	}

	if (EventType == AINPUT_EVENT_TYPE_MOTION)
	{
		int action = AMotionEvent_getAction(event);
		int actionType = action & AMOTION_EVENT_ACTION_MASK;
		size_t actionPointer = (size_t)((action & AMOTION_EVENT_ACTION_POINTER_INDEX_MASK) >> AMOTION_EVENT_ACTION_POINTER_INDEX_SHIFT);
		bool isActionTargeted = (actionType == AMOTION_EVENT_ACTION_POINTER_DOWN || actionType == AMOTION_EVENT_ACTION_POINTER_UP);
		int32 device = AInputEvent_getDeviceId(event);

		// trap Joystick events first, with fallthrough if there is no joystick support
		if (((EventSource & AINPUT_SOURCE_CLASS_JOYSTICK) == AINPUT_SOURCE_CLASS_JOYSTICK) &&
			(GetAxes != NULL) &&
			(actionType == AMOTION_EVENT_ACTION_MOVE))
		{
			const int axisCount = sizeof(AxisList)/sizeof(int32_t);

			// poll all the axes and forward to update controller state
			for (int axis = 0; axis < axisCount; axis++)
			{
				float val = GetAxes( event, AxisList[axis], 0);
				FAndroidInputInterface::JoystickAxisEvent(device, AxisList[axis], val);
			}

			// handle L/R trigger and Brake/Gas special (all in 0..1 range)
			// LTRIGGER will either be LTRIGGER or BRAKE, whichever is larger
			// RTRIGGER will either be RTRIGGER or GAS, whichever is larger
			float ltrigger = GetAxes(event, AMOTION_EVENT_AXIS_LTRIGGER, 0);
			float rtrigger = GetAxes(event, AMOTION_EVENT_AXIS_RTRIGGER, 0);
			float brake = GetAxes(event, AMOTION_EVENT_AXIS_BRAKE, 0);
			float gas = GetAxes(event, AMOTION_EVENT_AXIS_GAS, 0);
			FAndroidInputInterface::JoystickAxisEvent(device, AMOTION_EVENT_AXIS_LTRIGGER, ltrigger > brake ? ltrigger : brake);
			FAndroidInputInterface::JoystickAxisEvent(device, AMOTION_EVENT_AXIS_RTRIGGER, rtrigger > gas ? rtrigger : gas);

			return 1;
		}
		else
		{
			TArray<TouchInput> TouchesArray;
			
			TouchType type = TouchEnded;

			switch (actionType)
			{
			case AMOTION_EVENT_ACTION_DOWN:
			case AMOTION_EVENT_ACTION_POINTER_DOWN:
				type = TouchBegan;
				break;
			case AMOTION_EVENT_ACTION_MOVE:
				type = TouchMoved;
				break;
			case AMOTION_EVENT_ACTION_UP:
			case AMOTION_EVENT_ACTION_POINTER_UP:
			case AMOTION_EVENT_ACTION_CANCEL:
			case AMOTION_EVENT_ACTION_OUTSIDE:
				type = TouchEnded;
				break;
			case AMOTION_EVENT_ACTION_SCROLL:
			case AMOTION_EVENT_ACTION_HOVER_ENTER:
			case AMOTION_EVENT_ACTION_HOVER_MOVE:
			case AMOTION_EVENT_ACTION_HOVER_EXIT:
				return 0;
			default:
				UE_LOGF(LogAndroid, Verbose, "Unknown AMOTION_EVENT %d ignored", actionType);
				return 0;
			}

			size_t pointerCount = AMotionEvent_getPointerCount(event);

			if (pointerCount == 0)
			{
				return 1;
			}

			ANativeWindow* Window = (ANativeWindow*)FAndroidWindow::GetHardwareWindow_EventThread();
			if (!Window)
			{
				return 0;
			}

			int32_t Width = 0;
			int32_t Height = 0;

			if (Window)
			{
				// we are on the event thread. true here indicates we will retrieve dimensions from the current window.
				FAndroidWindow::CalculateSurfaceSize(Width, Height, true);
			}

			// make sure OpenGL context created before accepting touch events.. FAndroidWindow::GetScreenRect() may try to create it early from wrong thread if this is the first call
			if (!GAndroidGPUInfoReady)
			{
				return 1;
			}
			FPlatformRect ScreenRect = FAndroidWindow::GetScreenRect(true);

			if (AndroidThunkCpp_IsVirtualKeyboardShown() && (type == TouchBegan || type == TouchMoved))
			{
				//ignore key down events when the native input was clicked or when the keyboard animation is playing
				if (TryIgnoreClick(event, actionPointer))
				{
					return 0;
				}
			}
			else if ((AndroidThunkCpp_IsWebViewShown() || AndroidThunkCpp_HasExtraIgnoreView()) && (type == TouchBegan || type == TouchMoved || type == TouchEnded))
			{
				//ignore key down events when the the a web view is visible
				if (TryIgnoreClick(event, actionPointer) && ((EventSource & 0x80) != 0x80))
				{
					UE_LOGF(LogAndroid, Verbose, "Received touch event %d - Ignored", type);
					return 0;
				}
				UE_LOGF(LogAndroid, Verbose, "Received touch event %d", type);
			}
			if (isActionTargeted)
			{
				if (actionPointer < 0 || pointerCount < (int)actionPointer)
				{
					return 1;
				}

				int pointerId = AMotionEvent_getPointerId(event, actionPointer);
				float x = FMath::Min<float>(AMotionEvent_getX(event, actionPointer) / Width, 1.f);
				x *= (ScreenRect.Right - 1);
				float y = FMath::Min<float>(AMotionEvent_getY(event, actionPointer) / Height, 1.f);
				y *= (ScreenRect.Bottom - 1);

				UE_LOGF(LogAndroid, Verbose, "Received targeted motion event from pointer %zu (id %d) action %d: (%.2f, %.2f)", actionPointer, pointerId, action, x, y);

				TouchInput TouchMessage;
				TouchMessage.DeviceId = device;
				TouchMessage.Handle = pointerId;
				TouchMessage.Type = type;
				TouchMessage.Position = FVector2D(x, y);
				TouchMessage.LastPosition = FVector2D(x, y);		//@todo android: AMotionEvent_getHistoricalRawX
				TouchesArray.Add(TouchMessage);
			}
			else
			{
				for (size_t i = 0; i < pointerCount; ++i)
				{
					int pointerId = AMotionEvent_getPointerId(event, i);

					float x = FMath::Min<float>(AMotionEvent_getX(event, i) / Width, 1.f);
					x *= (ScreenRect.Right - 1);
					float y = FMath::Min<float>(AMotionEvent_getY(event, i) / Height, 1.f);
					y *= (ScreenRect.Bottom - 1);

					UE_LOGF(LogAndroid, Verbose, "Received motion event from index %zu (id %d) action %d: (%.2f, %.2f)", i, pointerId, action, x, y);

					TouchInput TouchMessage;
					TouchMessage.DeviceId = device;
					TouchMessage.Handle = pointerId;
					TouchMessage.Type = type;
					TouchMessage.Position = FVector2D(x, y);
					TouchMessage.LastPosition = FVector2D(x, y);		//@todo android: AMotionEvent_getHistoricalRawX
					TouchesArray.Add(TouchMessage);
				}
			}

			FAndroidInputInterface::QueueTouchInput(TouchesArray);

#if !UE_BUILD_SHIPPING
			if ((pointerCount >= 4) && (type == TouchBegan))
			{
				bool bShowConsole = true;
				GConfig->GetBool(TEXT("/Script/Engine.InputSettings"), TEXT("bShowConsoleOnFourFingerTap"), bShowConsole, GInputIni);

				if (bShowConsole)
				{
					GShowConsoleWindowNextTick = true;
					GDebugConsoleOpen = true;
				}
			}
#endif
		}

		return 0;
	}

	if (EventType == AINPUT_EVENT_TYPE_KEY)
	{
		int keyCode = AKeyEvent_getKeyCode(event);
		int keyFlags = AKeyEvent_getFlags(event);
		bool bSoftKey = (keyFlags & AKEY_EVENT_FLAG_SOFT_KEYBOARD) != 0;

		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Received keycode: %d, softkey: %d"), keyCode, bSoftKey ? 1 : 0);

		//Only pass on the device id if really a gamepad, joystick or dpad (allows menu and back to be treated as gamepad events)
		int32 device = -1;
		if ((((EventSource & AINPUT_SOURCE_JOYSTICK) == AINPUT_SOURCE_JOYSTICK) && (GetAxes != NULL)) ||
			((EventSource & AINPUT_SOURCE_GAMEPAD) == AINPUT_SOURCE_GAMEPAD) ||
			((EventSource & AINPUT_SOURCE_DPAD) == AINPUT_SOURCE_DPAD))
		{
			device = AInputEvent_getDeviceId(event);
		}

		//Trap codes handled as possible gamepad events
		if (device >= 0 && ValidGamepadKeyCodes.Contains(keyCode))
		{
			bool down = AKeyEvent_getAction(event) != AKEY_EVENT_ACTION_UP;
			FAndroidInputInterface::JoystickButtonEvent(device, keyCode, down);
			STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Received gamepad button: %d"), keyCode);
		}
		else
		{
			STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Received key event: %d"), keyCode);

			// only handle mapped key codes
			if (!MappedKeyCodes.Contains(keyCode))
			{
				return 0;
			}

			// forward key presses to UI if needed 
			if (AndroidThunkCpp_IsVirtualKeyboardShown())
			{
				return 0;
			}

			if (bSoftKey ||
				(GAndroidEnableHardwareKeyboard && !AndroidThunkCpp_IsVirtualKeyboardShown()) ||
				AlwaysAllowedKeyCodes.Contains(keyCode))
			{
				FDeferredAndroidMessage Message;

				Message.messageType = AKeyEvent_getAction(event) == AKEY_EVENT_ACTION_UP ? MessageType_KeyUp : MessageType_KeyDown;
				Message.KeyEventData.unichar = keyCode;
				Message.KeyEventData.keyId = keyCode;
				Message.KeyEventData.modifier = AKeyEvent_getMetaState(event);
				Message.KeyEventData.isRepeat = AKeyEvent_getAction(event) == AKEY_EVENT_ACTION_MULTIPLE;
				FAndroidInputInterface::DeferMessage(Message);
			}

			// allow event to be generated for volume up and down, but conditionally allow system to handle it, too
			if (keyCode == AKEYCODE_VOLUME_UP || keyCode == AKEYCODE_VOLUME_DOWN)
			{
				if (FPlatformMisc::GetVolumeButtonsHandledBySystem())
				{
					return 0;
				}
			}

			// optionally forward back button
			if (keyCode == AKEYCODE_BACK && GAllowJavaBackButtonEvent)
			{
				return 0;
			}
		}

		return 1;
	}

	return 0;
}

static bool IsStartupMoviePlaying()
{
	return GEngine && GEngine->IsInitialized() && GetMoviePlayer() && GetMoviePlayer()->IsStartupMoviePlaying();
}

static bool IsPreLoadScreenPlaying()
{
    return IsStartupMoviePlaying() 
        || (FPreLoadScreenManager::Get() && (FPreLoadScreenManager::Get()->HasValidActivePreLoadScreen()));
}

FAppEventData::FAppEventData(ANativeWindow* WindowIn)
{
	if (WindowIn == nullptr)
	{
		UE_LOGF(LogAndroid, Log, "FAppEventData was passed a NULL WindowsIn pointer.");
		return;
	}

	//check(WindowIn);
	if (GAndroidWindowOverride != nullptr )
	{
		GSurfaceViewWidth = WindowWidth = ANativeWindow_getWidth((ANativeWindow*)GAndroidWindowOverride);
		GSurfaceViewHeight = WindowHeight = ANativeWindow_getHeight((ANativeWindow*)GAndroidWindowOverride);
	}
	else
	{
		GSurfaceViewWidth = WindowWidth = ANativeWindow_getWidth(WindowIn);
		GSurfaceViewHeight = WindowHeight = ANativeWindow_getHeight(WindowIn);
	}

	if (WindowWidth <= 0)
	{
		UE_LOGF(LogAndroid, Log, "FAppEventData was passed a window with ZERO Height.");
		WindowWidth = 1;
	}
	if (WindowHeight <= 0)
	{
		UE_LOGF(LogAndroid, Log, "FAppEventData was passed a window with ZERO Height.");
		WindowHeight = 1;
	}
	//check(WindowWidth >= 0 && WindowHeight >= 0);
}

static void ActivateApp_EventThread()
{
	DEVELOPER_LOG_COMMANDCB_CASE(ActivateApp_EventThread);
	if (bAppIsActive_EventThread)
	{
		return;
	}

	if (!FAndroidMisc::UseNewWindowBehavior())
	{
		// Unlock window when we're ready.
		UE_LOGF(LogAndroid, Log, "event thread, activate app, unlocking HW window");
		GAndroidWindowLock.Unlock();
	}

	// wake the GT up.
	FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_APP_ACTIVATED);

	bAppIsActive_EventThread = true;
	FAppEventManager::GetInstance()->SetAppIsActive(bAppIsActive_EventThread);

	UE::GC::Private::SetSkipDestroy(false);
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("event thread, activate app, UNLOCKED HW window lock. bReadyToProcessEvents=%d, bAppIsActive_EventThread=%d"), bReadyToProcessEvents, bAppIsActive_EventThread);
	FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_RUN_CALLBACK, FAppEventData([]()
	{
		UE_LOGF(LogAndroid, Log, "performing app foregrounding callback.");
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Broadcast();
		FCoreDelegates::ApplicationHasReactivatedDelegate.Broadcast();
		FAppEventManager::GetInstance()->ResumeAudio();
	}));

	if (EventHandlerEvent)
	{
		// Must flush the queue before enabling rendering.
		EventHandlerEvent->Trigger();
	}

	if (FThreadHeartBeat* ThreadHeartBeat = FThreadHeartBeat::GetNoInit())
	{
		ThreadHeartBeat->ResumeHeartBeat(true);
	}

	FPreLoadScreenManager::EnableRendering(true);

	extern void AndroidThunkCpp_ShowHiddenAlertDialog();
	AndroidThunkCpp_ShowHiddenAlertDialog();	
}

extern void BlockRendering();

// called when the app has window with the new behavior model.
static void OnNewWindow_EventThread(ANativeWindow* NewNativeWindow, ANativeWindow* OldNativeWindow)
{
	// Unlock window when we're ready.
	UE_LOGF(LogAndroid, Log, "event thread, new window");

	TOptional<FAndroidWindow::FNativeAccessor> WindowAccessor = FAndroidWindowManager::Get().FindFromANativeWindow(true, true, OldNativeWindow);
	if (WindowAccessor.IsSet())
	{
		// Update the window handle, this will be used by the RHI callback.
		WindowAccessor->SetANativeWindow(NewNativeWindow);

		const auto& OnReinitWindowCallback = UE::FAndroidPlatformDynamicRHI::GetRHIOnReInitWindowCallback();
		if (OnReinitWindowCallback)
		{
			OnReinitWindowCallback(MoveTemp(WindowAccessor));
		}
	}
	else
	{
		// no window as yet, save the native handle for the next window create.
		FAndroidWindowManager::GetPendingWindowAccessor().SetANativeWindow(NewNativeWindow);
	}
}

// called whenever the app loses window with the new behavior model.
void OnLostWindow_EventThread(ANativeWindow* OutgoingWindow)
{
	TOptional<FAndroidWindow::FNativeAccessor> WindowAccessor = FAndroidWindowManager::Get().FindFromANativeWindow(true, false, OutgoingWindow);
	if (WindowAccessor.IsSet())
	{
		WindowAccessor->SetANativeWindow(nullptr);

		const auto& OnReleaseWindowCallback = UE::FAndroidPlatformDynamicRHI::GetRHIOnReleaseWindowCallback();
		if(OnReleaseWindowCallback)
		{
			OnReleaseWindowCallback(MoveTemp(WindowAccessor));
		}
	}
	else
	{
		FAndroidWindowManager::GetPendingWindowAccessor().SetANativeWindow(nullptr);
	}
}

// called whenever the app loses window or pause.
static void SuspendApp_EventThread()
{
	DEVELOPER_LOG_COMMANDCB_CASE(SuspendApp_EventThread);
	if (!bAppIsActive_EventThread)
	{
		UE_LOGF(LogAndroid, Warning, "SuspendApp_EventThread -> event thread, suspending app, bAppIsActive_EventThread is false so early return!");
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("SuspendApp_EventThread -> UNEXPECTED: event thread, SuspendApp_EventThread called but aborted, bReadyToProcessEvents=%d, bAppIsActive_EventThread=%d, bHasFocus=%d"), bReadyToProcessEvents, bAppIsActive_EventThread, bHasFocus);
		return;
	}
	bAppIsActive_EventThread = false;
	FAppEventManager::GetInstance()->SetAppIsActive(bAppIsActive_EventThread);

	UE::GC::Private::SetSkipDestroy(true);
	// Lock the window, this prevents event thread from removing the window whilst the RHI initializes.

	if (!FAndroidMisc::UseNewWindowBehavior())
	{
		UE_LOGF(LogAndroid, Log, "SuspendApp_EventThread -> event thread, suspending app, acquiring HW window lock.");
		GAndroidWindowLock.Lock();
	}
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("SuspendApp_EventThread -> event thread, suspending app, acquiring HW window lock. bReadyToProcessEvents=%d, bAppIsActive_EventThread=%d"), bReadyToProcessEvents, bAppIsActive_EventThread);

	if (bReadyToProcessEvents == false)
	{
		// App has stopped before we can process events.
		// AndroidLaunch will lock GAndroidWindowLock, and set bReadyToProcessEvents when we are able to block the RHI and queue up other events.
		// we ignore events until this point as acquiring GAndroidWindowLock means requires the window to be properly initialized.
		UE_LOGF(LogAndroid, Warning, "SuspendApp_EventThread -> UNEXPECTED: event thread, app not yet ready.");
		return;
	}

	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("SuspendApp_EventThread -> event thread, suspending app, ACQUIRED HW window lock. bReadyToProcessEvents=%d, bAppIsActive_EventThread=%d"), bReadyToProcessEvents, bAppIsActive_EventThread);

	TSharedPtr<FEvent, ESPMode::ThreadSafe> EMDoneTrigger = MakeShareable(FPlatformProcess::GetSynchEventFromPool(), [](FEvent* EventToDelete)
	{
		FPlatformProcess::ReturnSynchEventToPool(EventToDelete);
	});
	
	// perform the delegates before the window handle is cleared.
	// This ensures any tasks that require a window handle will have it before we block the RT on the invalid window.
	FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_RUN_CALLBACK, FAppEventData([EMDoneTrigger]()
	{
		UE_LOGF(LogAndroid, Log, "SuspendApp_EventThread -> performing app backgrounding callback. %p", EMDoneTrigger.Get());

		FCoreDelegates::ApplicationWillDeactivateDelegate.Broadcast();
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Broadcast();
		FAppEventManager::GetInstance()->PauseAudio();
		FAppEventManager::ReleaseMicrophone(false);

		EMDoneTrigger->Trigger();
	}));

#if USE_ANDROID_ALTERNATIVE_SUSPEND
		// Suspend the GT.
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_APP_SUSPENDED);
#endif

	uint32 StartCycles = FPlatformTime::Cycles();

	FEmbeddedCommunication::WakeGameThread();

	FPreLoadScreenManager::EnableRendering(false);

	FThreadHeartBeat::Get().SuspendHeartBeat(true);

	if (!FAndroidMisc::UseNewWindowBehavior())
	{
		// wait for a period of time before blocking rendering
		UE_LOGF(LogAndroid, Log, "SuspendApp_EventThread -> , waiting for event manager to process. tid: %d", FPlatformTLS::GetCurrentThreadId());

		bool bSuccess = EMDoneTrigger->Wait(4000);

		float ElapsedTimeInMs_EMDoneTrigger_Wait = FPlatformTime::ToMilliseconds(FPlatformTime::Cycles() - StartCycles);
		UE_CLOGF(!bSuccess, LogAndroid, Log, "SuspendApp_EventThread -> ERROR: backgrounding callback, not responded in timely manner. EMDoneTrigger->Wait, waited '%f' ms", (float)ElapsedTimeInMs_EMDoneTrigger_Wait);
		UE_LOGF(LogAndroid, Log, "SuspendApp_EventThread -> EMDoneTrigger->Wait, waited '%f' ms", (float)ElapsedTimeInMs_EMDoneTrigger_Wait);

		BlockRendering();
	}
	// Suspend the GT.
#if !USE_ANDROID_ALTERNATIVE_SUSPEND
	FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_APP_SUSPENDED);
#endif

	UE_LOGF(LogAndroid, Log, "SuspendApp_EventThread(EOF)");
}

//Called from the event process thread
static void OnAppCommandCB(struct android_app* app, int32_t cmd)
{
	check(IsInAndroidEventThread());

	// Since native app can only have one window we can track of the last handle.
	static ANativeWindow* LastDestroyedANativeWindow = nullptr;
	// Set event thread's view of the window dimensions:
	{
		ANativeWindow* DimensionWindow = app->pendingWindow ? app->pendingWindow : app->window;
		if (DimensionWindow)
		{
			FAndroidWindow::SetWindowDimensions_EventThread(DimensionWindow);
		}
	}

	switch (cmd)
	{
	case APP_CMD_SAVE_STATE:
		/**
		* Command from main thread: the app should generate a new saved state
		* for itself, to restore from later if needed.  If you have saved state,
		* allocate it with malloc and place it in android_app.savedState with
		* the size in android_app.savedStateSize.  The will be freed for you
		* later.
		*/
		// the OS asked us to save the state of the app
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_SAVE_STATE);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_SAVE_STATE");
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_SAVE_STATE);
		break;
	case APP_CMD_INIT_WINDOW:
		/**
		 * Command from main thread: a new ANativeWindow is ready for use.  Upon
		 * receiving this command, android_app->window will contain the new window
		 * surface.
		 */
		 // get the window ready for showing
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_INIT_WINDOW);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_INIT_WINDOW");

		if (FAndroidMisc::UseNewWindowBehavior())
		{
			OnNewWindow_EventThread(app->window, LastDestroyedANativeWindow);
		}
#if USE_ANDROID_STANDALONE
		{
			ANativeWindow* win = app->pendingWindow ? app->pendingWindow : app->window;
			STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Case APP_CMD_INIT_WINDOW(STANDALONE), app->pendingWindow=%p, app->window=%p, GAndroidWindowOverride=%p"), app->pendingWindow, app->window, GAndroidWindowOverride);
			if (GAndroidWindowOverride != NULL)
			{
				win = (ANativeWindow*)GAndroidWindowOverride;
			}
			checkf(win != NULL, TEXT("Engine APP_CMD_INIT_WINDOW with STANDALONE Failed since win is NULL!"));
			FAppEventManager::GetInstance()->HandleWindowCreated_EventThread((ANativeWindow*)win);
			bHasWindow = win != nullptr;
		}
#else
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Case APP_CMD_INIT_WINDOW"));
		FAppEventManager::GetInstance()->HandleWindowCreated_EventThread(app->pendingWindow);
		bHasWindow = true;
#endif
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_INIT_WINDOW: bHasWindow=%d, bHasFocus=%d, bIsResumed=%d", bHasWindow, bHasFocus, bIsResumed);
		if (!FAndroidMisc::UseNewWindowBehavior())
		{
			if (bHasWindow && bHasFocus && bIsResumed)
			{
				ActivateApp_EventThread();
			}
		}
		break;
	case APP_CMD_TERM_WINDOW:
		/**
		 * Command from main thread: the existing ANativeWindow needs to be
		 * terminated.  Upon receiving this command, android_app->window still
		 * contains the existing window; after calling android_app_exec_cmd
		 * it will be set to NULL.
		 */
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_TERM_WINDOW);
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Case APP_CMD_TERM_WINDOW, tid = %d"), gettid());
		// clean up the window because it is being hidden/closed
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_TERM_WINDOW");
		LastDestroyedANativeWindow = app->window;

		if (FAndroidMisc::UseNewWindowBehavior())
		{
			OnLostWindow_EventThread(app->window);
		}
		else
		{
			SuspendApp_EventThread();
		}

		FAppEventManager::GetInstance()->HandleWindowClosed_EventThread();
		bHasWindow = false;
		break;
	case APP_CMD_LOST_FOCUS:
		/**
		 * Command from main thread: the app's activity window has lost
		 * input focus.
		 */
		// if the app lost focus, avoid unnecessary processing (like monitoring the accelerometer)
		// log it, but the actual event will be simulated later in APP_CMD_PAUSE
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_LOST_FOCUS);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_LOST_FOCUS");

		break;
	case APP_CMD_GAINED_FOCUS:
		/**
		 * Command from main thread: the app's activity window has gained
		 * input focus.
		 */
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_GAINED_FOCUS);

		// bring back a certain functionality, like monitoring the accelerometer
		// log it, but the actual event will be simulated later in APP_CMD_RESUME
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_GAINED_FOCUS");

		// still check for a rare case needing activation
		if ((bHasWindow || FAndroidMisc::UseNewWindowBehavior()) && bHasFocus && bIsResumed)
		{
			ActivateApp_EventThread();
		}
		break;
	case APP_CMD_INPUT_CHANGED:
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_INPUT_CHANGED");
		break;
	case APP_CMD_WINDOW_RESIZED:
		/**
		 * Command from main thread: the current ANativeWindow has been resized.
		 * Please redraw with its new size.
		 */
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_WINDOW_RESIZED);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_WINDOW_RESIZED");
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_WINDOW_RESIZED, FAppEventData(app->window));
		break;
	case APP_CMD_WINDOW_REDRAW_NEEDED:
		/**
		 * Command from main thread: the system needs that the current ANativeWindow
		 * be redrawn.  You should redraw the window before handing this to
		 * android_app_exec_cmd() in order to avoid transient drawing glitches.
		 */
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_WINDOW_REDRAW_NEEDED);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_WINDOW_REDRAW_NEEDED");
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_WINDOW_REDRAW_NEEDED);
		break;
	case APP_CMD_CONTENT_RECT_CHANGED:
		/**
		 * Command from main thread: the content area of the window has changed,
		 * such as from the soft input window being shown or hidden.  You can
		 * find the new content rect in android_app::contentRect.
		 */
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_CONTENT_RECT_CHANGED);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_CONTENT_RECT_CHANGED");
		break;
	/* receive this event from Java instead to work around NDK bug with AConfiguration_getOrientation in Oreo
	case APP_CMD_CONFIG_CHANGED:
		{
			// Command from main thread: the current device configuration has changed.
			UE_LOGF(LogAndroid, Log, "Case APP_CMD_CONFIG_CHANGED");
			
			bool bPortrait = (AConfiguration_getOrientation(app->config) == ACONFIGURATION_ORIENTATION_PORT);
			if (FAndroidWindow::OnWindowOrientationChanged(bPortrait))
			{
				FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_WINDOW_CHANGED);
			}
		}
		break;
	*/
	case APP_CMD_LOW_MEMORY:
		/**
		 * Command from main thread: the system is running low on memory.
		 * Try to reduce your memory use.
		 */
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_LOW_MEMORY);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_LOW_MEMORY");
		break;
	case APP_CMD_START:
		/**
		 * Command from main thread: the app's activity has been started.
		 */
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_START);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_START");
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_ON_START);
	
		break;
	case APP_CMD_RESUME:

		/**
		 * Command from main thread: the app's activity has been resumed.
		 */
		bIsResumed = true;

		// assume focus on resume
		bHasFocus = true;
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_RESUME);

		if ((bHasWindow || FAndroidMisc::UseNewWindowBehavior()) && bHasFocus && bIsResumed)
		{
			ActivateApp_EventThread();
		}

		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Case APP_CMD_RESUME, bShouldRestartFromInterrupt=%d"), bShouldRestartFromInterrupt);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_RESUME");
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_ON_RESUME);

		// trigger focus
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_WINDOW_GAINED_FOCUS);

		/*
		* On the initial loading the restart method must be called immediately
		* in order to restart the app if the startup movie was playing
		*/
		if (bShouldRestartFromInterrupt)
		{
			AndroidThunkCpp_RestartApplication(TEXT(""));
		}
		break;
	case APP_CMD_PAUSE:
	{
		/**
		 * Command from main thread: the app's activity has been paused.
		 */
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_PAUSE);
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Case APP_CMD_PAUSE"));
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_PAUSE");

		// simulate lost focus
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_WINDOW_LOST_FOCUS);
		bHasFocus = false;

		// Ignore pause command for Oculus if the window hasn't been initialized to prevent halting initial load
		// if the headset is not active
		if (!bHasWindow && FAndroidMisc::GetDeviceMake() == FString("Oculus"))
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Oculus: Ignoring APP_CMD_PAUSE command before APP_CMD_INIT_WINDOW"));
			UE_LOGF(LogAndroid, Log, "Oculus: Ignoring APP_CMD_PAUSE command before APP_CMD_INIT_WINDOW");
			break;
		}

		bIsResumed = false;
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_ON_PAUSE);

		bool bAllowReboot = true;
#if FAST_BOOT_HACKS
		if (FEmbeddedDelegates::GetNamedObject(TEXT("LoggedInObject")) == nullptr)
		{
			bAllowReboot = false;
		}
#endif

		// Restart on resuming if did not complete engine initialization
		if (!bDidCompleteEngineInit && !bIgnorePauseOnDownloaderStart && bAllowReboot)
		{
// 			// only do this if early startup enabled
// 			FString *EarlyRestart = FAndroidMisc::GetConfigRulesVariable(TEXT("earlyrestart"));
// 			if (EarlyRestart != NULL && EarlyRestart->Equals("true", ESearchCase::IgnoreCase))
// 			{
// 				bShouldRestartFromInterrupt = true;
// 			}
		}
		bIgnorePauseOnDownloaderStart = false;

		/*
		 * On the initial loading the pause method must be called immediately
		 * in order to stop the startup movie's sound
		*/
		if (IsPreLoadScreenPlaying() && bAllowReboot)
		{
			UE_LOGF(LogAndroid, Log, "MoviePlayer force completion");
			GetMoviePlayer()->ForceCompletion();
		}

		SuspendApp_EventThread();

		break;
	}
	case APP_CMD_STOP:
		/**
		 * Command from main thread: the app's activity has been stopped.
		 */
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_STOP);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_STOP");
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_ON_STOP);
		break;
	case APP_CMD_DESTROY:
		/**
		* Command from main thread: the app's activity is being destroyed,
		* and waiting for the app thread to clean up and exit before proceeding.
		*/
		DEVELOPER_LOG_COMMANDCB_CASE(APP_CMD_DESTROY);
		UE_LOGF(LogAndroid, Log, "Case APP_CMD_DESTROY");

		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_RUN_CALLBACK, FAppEventData([]()
		{
			FGraphEventRef WillTerminateTask = FFunctionGraphTask::CreateAndDispatchWhenReady([]()
			{
				if (!UE::IsGCLocked())
				{
					FCoreDelegates::GetApplicationWillTerminateDelegate().Broadcast();
				}
			}, TStatId(), NULL, ENamedThreads::GameThread);
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(WillTerminateTask);
			FAndroidMisc::NonReentrantRequestExit();
		}));


		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_ON_DESTROY);

		// Exit here, avoids having to unlock the window and letting the RHI's deal with invalid window.
		extern void AndroidThunkCpp_ForceQuit();
		AndroidThunkCpp_ForceQuit();

		break;
	}

	if (EventHandlerEvent)
	{
		EventHandlerEvent->Trigger();
	}

	//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("#### END OF OnAppCommandCB cmd: %u, tid = %d"), cmd, gettid());
}

//Native-defined functions

jint JNICALL UE::Jni::FGameActivity::nativeGetCPUFamily(JNIEnv* jenv, jobject thiz)
{
	return (jint)android_getCpuFamily();
}

jboolean JNICALL UE::Jni::FGameActivity::nativeSupportsNEON(JNIEnv* jenv, jobject thiz)
{
	AndroidCpuFamily Family = android_getCpuFamily();

	if (Family == ANDROID_CPU_FAMILY_ARM64)
	{
		return JNI_TRUE;
	}
	if (Family == ANDROID_CPU_FAMILY_ARM)
	{
		return ((android_getCpuFeatures() & ANDROID_CPU_ARM_FEATURE_NEON) != 0) ? JNI_TRUE : JNI_FALSE;
	}
	return JNI_FALSE;
}

//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeOnOrientationChanged(int orientation);
void JNICALL UE::Jni::FGameActivity::nativeOnOrientationChanged(JNIEnv* jenv, jobject thiz, jint orientation)
{
	// enqueue a window changed event if orientation changed, 
	// note that the HW window handle does not necessarily change.
	if (FAndroidWindow::OnWindowOrientationChanged((EDeviceScreenOrientation)orientation))
	{	
		// Enqueue an event to trigger gamethread to update the orientation:
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_WINDOW_CHANGED);
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_SAFE_ZONE_UPDATED);

		if (EventHandlerEvent)
		{
			EventHandlerEvent->Trigger();
		}
	}
}

//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeConsoleCommand(String commandString);"
void JNICALL UE::Jni::FGameActivity::nativeConsoleCommand(JNIEnv* jenv, jobject thiz, jstring commandString)
{
#if !UE_BUILD_SHIPPING
	// TODO this is a very weak implementation, console popup can also be dismissed by pausing the app, improve it in the future
	GDebugConsoleOpen = false;
#endif

	FString Command = FJavaHelper::FStringFromParam(jenv, commandString);
#if !USE_ANDROID_STANDALONE
	if (GEngine != NULL)
	{
		// Run on game thread to avoid race condition with DeferredCommands
		AsyncTask(ENamedThreads::GameThread, [Command]()
		{
			GEngine->DeferredCommands.Add(Command);
		});
	}
	else
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("UNEXPECTED -> Ignoring console command (too early): %s"), *Command);
	}
#else
	IssueConsoleCommand(Command);
#endif

}

// This is called from the Java UI thread for initializing VR HMDs
void JNICALL UE::Jni::FGameActivity::nativeInitHMDs(JNIEnv* jenv, jobject thiz)
{
	for (auto HMDModuleIt = GHMDImplementations.CreateIterator(); HMDModuleIt; ++HMDModuleIt)
	{
		(*HMDModuleIt)->PreInit();
	}

	GHMDsInitialized = true;
}

void JNICALL UE::Jni::FGameActivity::nativeSetAndroidVersionInformation(JNIEnv* jenv, jobject thiz, jstring androidVersion, jint targetSDKversion, jstring phoneMake, jstring phoneModel, jstring phoneBuildNumber, jstring osLanguage, jstring productName)
{
	auto UEAndroidVersion = FJavaHelper::FStringFromParam(jenv, androidVersion);
	auto UEPhoneMake = FJavaHelper::FStringFromParam(jenv, phoneMake);
	auto UEPhoneModel = FJavaHelper::FStringFromParam(jenv, phoneModel);
	auto UEPhoneBuildNumber = FJavaHelper::FStringFromParam(jenv, phoneBuildNumber);
	auto UEOSLanguage = FJavaHelper::FStringFromParam(jenv, osLanguage);
	auto UEProductName = FJavaHelper::FStringFromParam(jenv, productName);

	FAndroidMisc::SetVersionInfo(UEAndroidVersion, targetSDKversion, UEPhoneMake, UEPhoneModel, UEPhoneBuildNumber, UEOSLanguage, UEProductName);
}

//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeOnInitialDownloadStarted();
void JNICALL UE::Jni::FGameActivity::nativeOnInitialDownloadStarted(JNIEnv* jenv, jobject thiz)
{
	bIgnorePauseOnDownloaderStart = true;
}

//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeOnInitialDownloadCompleted();
void JNICALL UE::Jni::FGameActivity::nativeOnInitialDownloadCompleted(JNIEnv* jenv, jobject thiz)
{
	bIgnorePauseOnDownloaderStart = false;
}

// MERGE-TODO: Anticheat concerns with custom input
bool GAllowCustomInput = true;
void JNICALL UE::Jni::FNativeCalls::HandleCustomTouchEvent(JNIEnv* jenv, jclass clazz, jint deviceId, jint pointerId, jint action, jint soucre, jfloat x, jfloat y)
{
#if ANDROID_ALLOWCUSTOMTOUCHEVENT
	// make sure fake input is allowed, so hacky Java can't run bots
	if (!GAllowCustomInput)
	{
		return;
	}

	TArray<TouchInput> TouchesArray;

	TouchInput TouchMessage;
	TouchMessage.DeviceId = deviceId;
	TouchMessage.Handle = pointerId;
	switch (action) {
	case 0: //  MotionEvent.ACTION_DOWN
		TouchMessage.Type = TouchBegan;
		break;
	case 2: // MotionEvent.ACTION_MOVE
		TouchMessage.Type = TouchMoved;
		break;
	default: // MotionEvent.ACTION_UP
		TouchMessage.Type = TouchEnded;
		break;
	}
	TouchMessage.Position = FVector2D(x, y);
	TouchMessage.LastPosition = FVector2D(x, y);

	TouchesArray.Add(TouchMessage);

	UE_LOGF(LogAndroid, Verbose, "Handle custom touch event %d (%d) x=%f y=%f", TouchMessage.Type, action, x, y);
	FAndroidInputInterface::QueueTouchInput(TouchesArray);
#endif
}

void JNICALL UE::Jni::FNativeCalls::AllowJavaBackButtonEvent(JNIEnv* jenv, jclass clazz, jboolean allow)
{
	GAllowJavaBackButtonEvent = (allow == JNI_TRUE);
}

bool WaitForAndroidLoseFocusEvent(double TimeoutSeconds)
{
	return FAppEventManager::GetInstance()->WaitForEventInQueue(EAppEventState::APP_EVENT_STATE_WINDOW_LOST_FOCUS, TimeoutSeconds);
}

#if USE_ANDROID_STANDALONE
pthread_t G_AndroidMainThread;

static struct android_app dummy_state;

#define EVENTTYPE_INIT						0
#define EVENTTYPE_POST_ENGINE_INIT			1
#define EVENTTYPE_ENGINELOOP_INIT_COMPLETE	2
#define EVENTTYPE_FRAME_BEGIN				3
#define EVENTTYPE_FRAME_END					4
#define EVENTTYPE_PRE_LOAD_MAP				5
#define EVENTTYPE_POST_LOAD_MAP				6
#define EVENTTYPE_ACTION					7
#define EVENTTYPE_ACTIVITYCHANGED			8
#define EVENTTYPE_APP_SUSPENDED				9


static const TCHAR* eAppCommandNamed[] = {
	TEXT("APP_CMD_INPUT_CHANGED"),
	TEXT("APP_CMD_INIT_WINDOW"),
	TEXT("APP_CMD_TERM_WINDOW"),
	TEXT("APP_CMD_WINDOW_RESIZED"),
	TEXT("APP_CMD_WINDOW_REDRAW_NEEDED"),
	TEXT("APP_CMD_CONTENT_RECT_CHANGED"),
	TEXT("APP_CMD_GAINED_FOCUS"),
	TEXT("APP_CMD_LOST_FOCUS"),
	TEXT("APP_CMD_CONFIG_CHANGED"),
	TEXT("APP_CMD_LOW_MEMORY"),
	TEXT("APP_CMD_START"),
	TEXT("APP_CMD_RESUME"),
	TEXT("APP_CMD_SAVE_STATE"),
	TEXT("APP_CMD_PAUSE"),
	TEXT("APP_CMD_STOP"),
	TEXT("APP_CMD_DESTROY"),
	TEXT("APP_CMD_UNKNOWN")
};

static const TCHAR* get_APP_CMD_String(uint32 cmd)
{
	check(cmd <= 15);
	return (cmd <= 15) ? eAppCommandNamed[cmd] : eAppCommandNamed[16];
}

void AndroidThunkCpp_Engine_SendEvent(int32 event, const FString& param1, int32 param2, int32 param3, float param4)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jclass EngineClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/asis/Engine");
		static jmethodID SendEventFunc = FJavaWrapper::FindStaticMethod(Env, EngineClass, "AndroidThunkJava_Engine_ReceiveEvent", "(ILjava/lang/String;IIF)V", false);

		if (SendEventFunc != nullptr)
		{
			auto StringParam = FJavaHelper::ToJavaString(Env, param1);
			Env->CallStaticVoidMethod(EngineClass, SendEventFunc, event, *StringParam, param2, param3, param4);
		}
		else
		{
			STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidThunkCpp_Engine_SendEvent WARNING: SendEventFunc not found"));

		}
	}
	else
	{

		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidThunkCpp_Engine_SendEvent ERROR: FAndroidApplication::GetJavaEnv() == null"));
	}
}

static void InitEvent()
{
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("LaunchAndroid received InitEvent"));
	AndroidThunkCpp_Engine_SendEvent(EVENTTYPE_INIT, TEXT(""), 0, 0, 0.0f);
}

static void AppSuspended()
{
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("LaunchAndroid received AppSuspended, GFrameCounter=%d"), (int32)GFrameCounter);
	AndroidThunkCpp_Engine_SendEvent(EVENTTYPE_APP_SUSPENDED, TEXT(""), (int32)GFrameCounter, 0, 0.0f);
}

static void PostEngineInitEvent()
{
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("LaunchAndroid received PostEngineInitEvent"));

	AndroidThunkCpp_Engine_SendEvent(EVENTTYPE_POST_ENGINE_INIT, TEXT(""), 0, 0, 0.0f);
}

static void EngineLoopInitCompleteEvent()
{
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("LaunchAndroid received EngineLoopInitCompleteEvent"));
	AndroidThunkCpp_Engine_SendEvent(EVENTTYPE_ENGINELOOP_INIT_COMPLETE, TEXT(""), 0, 0, 0.0f);
}

static void BeginFrameEvent()
{
	AndroidThunkCpp_Engine_SendEvent(EVENTTYPE_FRAME_BEGIN, TEXT(""), (int32)GFrameCounter, 0, 0.0f);
}

static void EndFrameEvent()
{
	AndroidThunkCpp_Engine_SendEvent(EVENTTYPE_FRAME_END, TEXT(""), (int32)(GFrameCounter - 1), 0, 0.0f);
}

static void PreLoadMapEvent(const FString& Param)
{
	AndroidThunkCpp_Engine_SendEvent(EVENTTYPE_PRE_LOAD_MAP, Param, 0, 0, 0.0f);
}

static void PostLoadMapEvent(UWorld* World)
{
	if (World != nullptr)
	{
		FString LevelName = World->GetMapName();
		AndroidThunkCpp_Engine_SendEvent(EVENTTYPE_POST_LOAD_MAP, LevelName, 0, 0, 0.0f);
	}
	else
	{
		AndroidThunkCpp_Engine_SendEvent(EVENTTYPE_POST_LOAD_MAP, TEXT(""), 0, 0, 0.0f);
	}
}

void JNICALL UE::Jni::Asis::FAsisGameActivity::nativeSetCommandline(JNIEnv* jenv, jclass clazz, jstring commandline)
{
	auto UECommandLine = FJavaHelper::FStringFromParam(jenv, commandline);
	GAndroidCommandLine = UECommandLine;
}

void JNICALL UE::Jni::Asis::FAsisGameActivity::nativeMain(JNIEnv* jenv, jclass clazz, jstring projectModule)
{
	BootTimingPoint("native_main");
	GIsGameAgnosticExe = true;

	FString projectModuleName = FJavaHelper::FStringFromParam(jenv, projectModule);

	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("UE::Jni::Asis::FAsisGameActivity::nativeMain : use current!, requesting ProjectModule: %s"), *projectModuleName);

	if (EventThreadID == 0)
	{
		FTaskTagScope::SwapTag(ETaskTag::EEventThread);
		EventThreadID = FPlatformTLS::GetCurrentThreadId();
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidMain set current EventThreadID=%d"), EventThreadID);
	}

	// register some delegates we want to pass back
	FCoreDelegates::OnInit.AddStatic(InitEvent);
	FCoreDelegates::GetOnPostEngineInit().AddStatic(PostEngineInitEvent);
	FCoreDelegates::OnFEngineLoopInitComplete.AddStatic(EngineLoopInitCompleteEvent);
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddStatic(AppSuspended);
	FCoreDelegates::OnBeginFrame.AddStatic(BeginFrameEvent);
	FCoreDelegates::OnEndFrame.AddStatic(EndFrameEvent);
	FCoreUObjectDelegates::PreLoadMap.AddStatic(PreLoadMapEvent);
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddStatic(PostLoadMapEvent);

	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Entering engine main function"));

	dummy_state.activity = nullptr;

	pthread_attr_t otherAttr;
	pthread_attr_init(&otherAttr);
	pthread_attr_setdetachstate(&otherAttr, PTHREAD_CREATE_DETACHED);
	pthread_create(&G_AndroidMainThread, &otherAttr, AndroidMain, &dummy_state);

	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("Created main thread"));
}

static bool MountPak(const FString& PakFilePath, int32 PakOrder, const FString& PakMountPoint)
{
	// Mount all pak files found in this content
	FPakPlatformFile* PakFileMgr = (FPakPlatformFile*)(FPlatformFileManager::Get().FindPlatformFile(TEXT("PakFile")));
	if (PakFileMgr == nullptr)
	{
		return false;
	}

	if (!PakMountPoint.IsEmpty())
	{
		const TCHAR* MountPoint = PakMountPoint.GetCharArray().GetData();
		return PakFileMgr->Mount(*PakFilePath, PakOrder, MountPoint);
	}

	return PakFileMgr->Mount(*PakFilePath, PakOrder);
}

JNI_METHOD bool Java_com_epicgames_asis_Engine_nativeMountPak(JNIEnv* jenv, jobject thiz, jstring pakFile, jint order, jstring mountPoint)
{
	FString PakFilePath = FJavaHelper::FStringFromParam(jenv, pakFile);
	uint32 PakOrder = (uint32)FMath::Max(0, order);
	FString PakMountPoint = FJavaHelper::FStringFromParam(jenv, mountPoint);

	if (!IsInGameThread() && FTaskGraphInterface::IsRunning())
	{
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("nativeMountPak: deferred '%s'"), *PakFilePath);
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateLambda([=]()
				{
					MountPak(PakFilePath, PakOrder, PakMountPoint);
				}), TStatId(), nullptr, ENamedThreads::GameThread);
		return true;
	}
	else
	{
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("nativeMountPak: immediate '%s'"), *PakFilePath);
		return MountPak(PakFilePath, PakOrder, PakMountPoint);
	}
}

void JNICALL UE::Jni::Asis::FAsisGameActivity::nativeAppCommand(JNIEnv* jenv, jclass clazz, jint cmd)
{
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AsisGameActivity -> nativeAppCommand[%u, %s] -> EventThreadID=%d, GGameThreadId=%d, GResumeMainInit=%d, GEventHandlerInitialized=%d, FTaskGraphInterface::IsRunning()=%d, FPlatformTLS::GetCurrentThreadId()=%d"), 
		cmd,
		get_APP_CMD_String(cmd),
		EventThreadID, 
		GGameThreadId,
		GResumeMainInit,
		GEventHandlerInitialized,
		FTaskGraphInterface::IsRunning(),
		FPlatformTLS::GetCurrentThreadId()
	);

	dummy_state.window = (ANativeWindow*)GAndroidWindowOverride;
	dummy_state.pendingWindow = (ANativeWindow*)GAndroidWindowOverride;

	OnAppCommandCB(&dummy_state, cmd);
}

#endif // USE_ANDROID_STANDALONE


//Wraps __cxa_atexit and avoids registering any destructors for static objects to save startup time
extern "C" {
    int __wrap___cxa_atexit(void (*f)(void*), void* objptr, void* dso)
    {
	    return 0;
    }
};

#endif //  USE_ANDROID_LAUNCH
