// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_MAC

#include "../../../Launch/Public/Mac/UEAppDelegate.h"

void KickoffEngine();

#else // mobile 

// define this so we disable C++ etc bits swift doesn't beed
#define SWIFT_IMPORT
#include "../../../ApplicationCore/Public/IOS/IOSAppDelegate.h"
#include "../../../ApplicationCore/Public/IOS/IOSView.h"

#endif


// setup the kickoff function
#if PLATFORM_VISIONOS

#import <CompositorServices/CompositorServices.h>
void KickoffWithCompositingLayer(CP_OBJECT_cp_layer_renderer* Layer);

#else

void KickoffEngine();
#endif
