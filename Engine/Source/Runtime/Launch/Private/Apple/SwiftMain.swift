import SwiftUI
import MetalKit

#if os(visionOS)
import CompositorServices

struct SwiftUIView: View {
	var onClick: () -> Void = {}

	@Environment(\.openImmersiveSpace) var openImmersiveSpace
	@Environment(\.dismissWindow) private var dismissWindow

	@State private var buttonDisabled = true
	@State private var timerCounter = 3
	@State private var timerText = "3"
	let timer = Timer.publish(every: 1, on: .main, in: .common).autoconnect()
	@State private var messageText = "Waiting for configuration..."
	
	@Binding var style: ImmersionStyle
	@Binding var limb: Visibility
	
	func Open()
	{
		print("SwiftUIView Open()")

		buttonDisabled = true
		timer.upstream.connect().cancel()

		Task
		{
			await openImmersiveSpace(id: "ImmersiveSpace")
			
			dismissWindow(id: "SwiftUIView")
		}
	}
	
	var body: some View {
		VStack(spacing: 8) {
			Text("Unreal SwiftMain Splash Screen!")
				.font(.title)
				.bold()
			Text(messageText)
			Button(action: {
				Open()
			}
				   , label: {
				Text("Go Immersive!")
			})
			.disabled(buttonDisabled)
			Text("Automatically go in \(timerText)")
				.id("Countdown")
		}
		.onReceive(timer) { t2 in
			timerCounter -= 1	
			timerText = "\(timerCounter)"
			if timerCounter == 0 {
				Open()
			}
		}
		.onReceive(NotificationCenter.default.publisher(for: .configureImmersiveSpace)) { notification in
			guard let userInfo = notification.userInfo else { return }
			
			if let styleValue = userInfo["immersiveStyle"] as? Int32 {
				switch styleValue {
				case 0:
					style = .full
					break
				case 1:
					style = .mixed
					break
				default:
					style = .full
				}
			}

			if let upperLimbVisibilityValue = userInfo["upperLimbVisibility"] as? Int32 {
				switch upperLimbVisibilityValue {
				case 0:
					limb = .hidden
					break
				case 1:
					limb = .visible
					break
#if !UE_SDK_VERSION_1
// sdk 1 does not support automatic, so fall back to visible
				case 2:
					limb = .automatic
					break
#endif
				default:
					limb = .visible			
				}
			}
			
			messageText = "ImmersiveStyle: \(String(describing:style))  UpperLimbVisibility: \(String(describing:limb)) "
			print("messageText: \(messageText)")
			
			buttonDisabled = false
		}
	}
}

struct UEContentConfiguration: CompositorLayerConfiguration {
	func makeConfiguration(
		capabilities: LayerRenderer.Capabilities,
		configuration: inout LayerRenderer.Configuration
	)
	{
		/*
		//let supportsFoveation = capabilites.supportsFoveation
		//let supportedLayouts = supportedLayouts(options: supportsFoveation ? [.foveationEnabled] : [])
		let supportedColorFormats = capabilities.supportedColorFormats
		let supportedDepthFormats = capabilities.supportedDepthFormats
		print("Supported Color Formats: ")
		supportedColorFormats.forEach { colorFormat in
			print(colorFormat.rawValue)
		}
		print("Supported Depth Formats: ")
		supportedDepthFormats.forEach { depthFormat in
			print(depthFormat.rawValue)
		}
		*/
		
		configuration.layout = .shared
		//configuration.layout = .dedicated // separate texture for each eye.  We may need to switch when we implement foveated rendering.
		configuration.isFoveationEnabled = false
		
		// HDR support // Might want to switch based on project settings.
		//configuration.colorFormat = .rgba16Float
		configuration.colorFormat = .bgra8Unorm_srgb
		
		//configuration.depthFormat = .depth32Float  			//PF_R32_FLOAT   	
		configuration.depthFormat = .depth32Float_stencil8 		//PF_DepthStencil 

		
		configuration.defaultDepthRange = [Float.greatestFiniteMagnitude, 0.1]
	}
}

// unused at the moment, but this is how UE can open a SwiftUI view form the Obj-C side
//class HostingViewFactory: NSObject {
//	static var onStartImmersive: (LayerRenderer) -> Void = { layer in }
//
//	@objc static func MakeSwiftUIView(OnClick: @escaping (() -> Void)) -> UIViewController
//	{
//	  return UIHostingController(rootView: SwiftUIView(onClick: OnClick))
//	}
//}

// Bridge class for calling into this file from c++
// Data comes in from SwiftMainBridge.cpp and goes out via the swift notification system to SwiftUI elements.
@objc class SwiftMainBridge: NSObject {

	@objc static func configureImmersiveSpace(inImmersiveStyle: Int32, inUpperLimbVisibility: Int32) {
		NotificationCenter.default.post(name: .configureImmersiveSpace, 
										object: nil, 
										userInfo: ["immersiveStyle": inImmersiveStyle, "upperLimbVisibility": inUpperLimbVisibility])
	}
}

extension Notification.Name {
	static let configureImmersiveSpace = Notification.Name("configureImmersiveSpace")
}

#if UE_USE_SWIFT_UI_MAIN

@main
struct UESwiftApp: App {
	
	@State private var style: ImmersionStyle = .full
	@State private var limb: Visibility = .visible
	
	@UIApplicationDelegateAdaptor(IOSAppDelegate.self) var delegate
	
	var body: some Scene
	{
		WindowGroup(id: "SwiftUIView")
		{
			SwiftUIView(style: $style, limb: $limb)
		}

		ImmersiveSpace(id: "ImmersiveSpace")
		{
			// this will make a CompositorLayer that can pull a Metal drawable out for UE to render to
			CompositorLayer(configuration: UEContentConfiguration())
			{
				// on button click, tell Unreal the layer is ready and can continue
				layerRenderer in
					KickoffWithCompositingLayer(layerRenderer)
			}
		}
		.immersionStyle(selection: $style, in: .mixed, .full)
		.upperLimbVisibility($limb.wrappedValue)
	}
}

#endif

#else

#if os(iOS) || os(tvOS)
struct UEView: UIViewRepresentable {

	func makeUIView(context: Context) -> FIOSView {
		FIOSView()
	}
	func updateUIView(_ uiView: FIOSView, context: Context) {
//		uiView.attributedText = text
	}
}
#endif

struct SwiftUIView: View {
	var onClick: () -> Void = {}

	@State private var buttonDisabled = false

	func Open()
	{
//		print("Layer: \(self.layer)");
		Task
		{
			buttonDisabled = true
			KickoffEngine();
		}
	}
	
//	let timer = Timer.publish(every: 3, on: .main, in: .common).autoconnect()
	
	var body: some View {
		//		print("body view \(self)");
		//		MTKView(frame: self.frame, device: MTLDevice.preferredDevice)
		ZStack() {
#if os(iOS) || os(tvOS)
			UEView()
#endif
			VStack(spacing: 8) {
				Text("SwiftUI in Unreal!")
					.font(.title)
					.bold()
				Button(action: {
					Open()
				}
					   , label: {
					Text("Test Button")
				})
				.disabled(buttonDisabled)
				
			}
			.opacity(buttonDisabled ? 0.0 : 1.0)
			//	.onReceive(timer) { t2 in
			//		print("timer!")
			//		timer.upstream.connect().cancel()
			//		Open()
			//	}
		}
	}
}

#if UE_USE_SWIFT_UI_MAIN

@main
struct BridgeTestApp: App {
	
#if os(macOS)
	@NSApplicationDelegateAdaptor(UEAppDelegate.self) var delegate
#else
	@UIApplicationDelegateAdaptor(IOSAppDelegate.self) var delegate
#endif
	
	var body: some Scene {
		WindowGroup {
			SwiftUIView()
		}
	}
}

//#Preview {
//    SwiftUIView()
//}

#endif

#endif





