// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwiftMainBridge.h"

#if PLATFORM_VISIONOS

// This import is the autogenerated "bridging header" created by xcode because SwiftMain.swift in the Launch module contains @objc tagged items.
// Look in there if you need to understand the objective c interface better.
#import "Launch-Swift.h"

// All our calls come in from unreal threads, so we dispatch_async to get the into Apple's main thread where SwiftUI updates must happen.

void UE::SwiftMainBridgeNS::ConfigureImmersiveSpace(int32 InImmersiveStyle, int32 InUpperLimbVisibility)
{
	dispatch_async(dispatch_get_main_queue(), ^{
		[SwiftMainBridge configureImmersiveSpaceInImmersiveStyle :InImmersiveStyle inUpperLimbVisibility:InUpperLimbVisibility];
	});

}

#endif // PLATFORM_VISIONOS
