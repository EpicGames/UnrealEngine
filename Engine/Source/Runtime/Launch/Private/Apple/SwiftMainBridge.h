// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

// Bridge functions for communicating from C++ to SwiftMain.swift.
// See LaunchCPPToSwift.h for an example of going the other way, from SwiftMain.swift to C++.

#if PLATFORM_VISIONOS

namespace UE::SwiftMainBridgeNS
{
	void ConfigureImmersiveSpace(int32 InImmersiveStyle, int32 InUpperLimbVisibility);
}

#endif // PLATFORM_VISIONOS
