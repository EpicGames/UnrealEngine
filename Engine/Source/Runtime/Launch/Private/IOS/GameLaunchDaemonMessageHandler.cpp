// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameLaunchDaemonMessageHandler.h"

#include "HAL/PlatformProcess.h"
#include "IMessageContext.h"
#include "IOSMessageProtocol.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include <UIKit/UIDevice.h>

#import <TargetConditionals.h>


void FGameLaunchDaemonMessageHandler::Init()
{
    FModuleManager::Get().LoadModuleChecked(TEXT("LaunchDaemonMessages"));
    
	MessageEndpoint = FMessageEndpoint::Builder("FGameLaunchDaemonMessageHandler")
		.Handling<FIOSLaunchDaemonPing>(this, &FGameLaunchDaemonMessageHandler::HandlePingMessage)
		.Handling<FIOSLaunchDaemonLaunchApp>(this, &FGameLaunchDaemonMessageHandler::HandleLaunchRequest);

	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Subscribe<FIOSLaunchDaemonPing>();
	}
}


void FGameLaunchDaemonMessageHandler::Shutdown()
{
	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint.Reset();
	}
}


void FGameLaunchDaemonMessageHandler::HandlePingMessage(const FIOSLaunchDaemonPing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (MessageEndpoint.IsValid())
	{
		FMessageAddress MessageSender = Context->GetSender();

		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FIOSLaunchDaemonPong>(FString(FPlatformProperties::PlatformName()) + (TARGET_IPHONE_SIMULATOR ? FString("Simulator:") : FString("@")) + FString(FPlatformProcess::ComputerName())
			, FString("udid")
			, FPlatformProcess::ComputerName()
			, "Game_Running"
			, [[UIDevice currentDevice] userInterfaceIdiom] == UIUserInterfaceIdiomPhone? "Phone" : "Tablet"
			, FString("local")
			, false
			, false
			, false
			, true
			), MessageSender);
	}
}


void FGameLaunchDaemonMessageHandler::HandleLaunchRequest(const FIOSLaunchDaemonLaunchApp& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FString Error;

	// We're in the game. We need to launch ULD instead, with launch arguments that tell it to immediately relaunch.
	FString NewURL = "UnrealLaunchDaemon:// -directlaunch ";
	NSLog(@"Launching ULD\n");

	// Leaving in extra variable in case text debugging needs to be injected.
	FString LaunchURL = NewURL + FString(Message.AppID + TEXT("://") + Message.Parameters);
	FPlatformProcess::LaunchURL(*LaunchURL, NULL, &Error);

	// Exiting the process prevents a sockets conflict with the game. 
	// Here's the supported sequence of events:
	// 1) Launch ULD manually to kick things off
	// 2) UFE wants to launch the game and sends the LaunchRequest.
	// 3a) If ULD is running, it launches the game and shuts down
	// 3b) If the game is running, it launches ULD with special arguments and shuts down. 
	//     ULD detects that it needs to do an immediate launch, and after a few second delay to let the game shut down,
	//     it relaunches the game.
	// 4) rinse and repeat.
	//
	// It's worth mentioning that exit(0) is not considered an appropriate way for shutting down a consumer app. But in this case, it works fine.
	exit(0);
}
