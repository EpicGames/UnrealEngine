// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

class FMessageEndpoint;
class IMessageContext;

struct FIOSLaunchDaemonLaunchApp;
struct FIOSLaunchDaemonPing;


class FGameLaunchDaemonMessageHandler
{
public:

	void Init();
	void Shutdown();

private:

	void HandlePingMessage(const FIOSLaunchDaemonPing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);
	void HandleLaunchRequest(const FIOSLaunchDaemonLaunchApp& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

private:

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	FString AppId;
};
