// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSAppDelegateConsoleHandling.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"

extern bool GShowSplashScreen;

#if !UE_BUILD_SHIPPING
extern bool GDebugConsoleOpen;
#endif

@class IOSViewController;

@implementation IOSAppDelegate (ConsoleHandling)

#if !UE_BUILD_SHIPPING && !PLATFORM_TVOS
/** 
 * Shows the console and brings up an on-screen keyboard for input
 */
- (void)ShowConsole
{
	if (self.ConsoleAlertController != nil)
	{
		return;
	}

	GDebugConsoleOpen = true;

	// start at the end of the list for history
	self.ConsoleHistoryValuesIndex = [self.ConsoleHistoryValues count];

	// Set up a containing alert message and buttons
	if ([UIAlertController class])
	{
		self.ConsoleAlertController = [UIAlertController alertControllerWithTitle : @""
												message:@"Type a console command"
												preferredStyle:UIAlertControllerStyleAlert];

		UIAlertAction* okAction = [UIAlertAction 
									actionWithTitle:NSLocalizedString(@"OK", nil)
									style:UIAlertActionStyleDefault
									handler:^(UIAlertAction* action)
									{
										self.AlertResponse = 1;
										[self.ConsoleAlertController dismissViewControllerAnimated : YES completion : nil];

										// we clicked Ok (not Cancel at index 0), submit the console command
                                        UITextField* AlertTextField = self.ConsoleAlertController.textFields.firstObject;
										[self HandleConsoleCommand:AlertTextField.text];

										self.ConsoleAlertController = nil;
										GDebugConsoleOpen = false;
									}
		];
		UIAlertAction* cancelAction = [UIAlertAction
										actionWithTitle : NSLocalizedString(@"Cancel", nil)
										style : UIAlertActionStyleDefault
										handler : ^ (UIAlertAction* action)
										{
											self.AlertResponse = 0;
											[self.ConsoleAlertController dismissViewControllerAnimated : YES completion : nil];
											self.ConsoleAlertController = nil;
											GDebugConsoleOpen = false;
										}
		];

		[self.ConsoleAlertController addAction : okAction];
		[self.ConsoleAlertController addAction : cancelAction];
		[self.ConsoleAlertController
			addTextFieldWithConfigurationHandler : ^ (UITextField* AlertTextField)
			{
				AlertTextField.clearsOnBeginEditing = NO;
				AlertTextField.autocorrectionType = UITextAutocorrectionTypeNo;
				AlertTextField.autocapitalizationType = UITextAutocapitalizationTypeNone;
				AlertTextField.placeholder = @"or swipe for history";
				AlertTextField.clearButtonMode = UITextFieldViewModeWhileEditing;
				AlertTextField.delegate = self;
				AlertTextField.clearsOnInsertion = NO;
				AlertTextField.keyboardType = UIKeyboardTypeDefault;

				// Add gesture recognizers
				UISwipeGestureRecognizer* SwipeLeftGesture = [[UISwipeGestureRecognizer alloc] initWithTarget:self action : @selector(SwipeLeftAction:)];
				SwipeLeftGesture.direction = UISwipeGestureRecognizerDirectionLeft;
				[AlertTextField addGestureRecognizer : SwipeLeftGesture];

				UISwipeGestureRecognizer* SwipeRightGesture = [[UISwipeGestureRecognizer alloc] initWithTarget:self action : @selector(SwipeRightAction:)];
				SwipeRightGesture.direction = UISwipeGestureRecognizerDirectionRight;
				[AlertTextField addGestureRecognizer : SwipeRightGesture];
			}
		];
//		[self.ConsoleAlertController release];
		[[IOSAppDelegate GetDelegate].IOSController presentViewController : self.ConsoleAlertController animated : YES completion : nil];
	}
}
#endif

#if !UE_BUILD_SHIPPING
/**
 * Handles processing of an input console command
 */
- (void)HandleConsoleCommand:(NSString*)ConsoleCommand
{
	if ([ConsoleCommand length] > 0)
	{
		if (self.bEngineInit && GEngine != nullptr)
		{
			TArray<TCHAR> Ch;
			Ch.AddZeroed([ConsoleCommand length]+1);

			FPlatformString::CFStringToTCHAR((CFStringRef)ConsoleCommand, Ch.GetData());
			GEngine->DeferredCommands.Add(FString(Ch.GetData()));
		}
#if !PLATFORM_TVOS
		NSUInteger ExistingCommand = [self.ConsoleHistoryValues indexOfObjectPassingTest:
			^ BOOL (id obj, NSUInteger idx, BOOL *stop)
			{ 
				return [obj caseInsensitiveCompare:ConsoleCommand] == NSOrderedSame; 
			}
		];

		// remove the existing one, so we can move it to the end
		if (ExistingCommand != NSNotFound)
		{
			[self.ConsoleHistoryValues removeObjectAtIndex:ExistingCommand];
		}

		// add the command to the end
		[self.ConsoleHistoryValues addObject:ConsoleCommand];
		// save to local storage
		[[NSUserDefaults standardUserDefaults] setObject:self.ConsoleHistoryValues forKey:@"ConsoleHistory"];
		[[NSUserDefaults standardUserDefaults] synchronize];
#endif
	}
}
#endif

/** 
 * Shows an alert with up to 3 buttons. A delegate callback will later set AlertResponse property
 */
- (void)ShowAlert:(NSMutableArray*)StringArray
{
	UIViewController *ViewController = (UIViewController*)[IOSAppDelegate GetDelegate].IOSController;
	if (GShowSplashScreen)
	{
		if ([[IOSAppDelegate GetDelegate].Window viewWithTag : 200] != nil)
		{
			// If the splash screen is showing, the UIView isn't valid yet, so check for a ViewController
			if(ViewController == nil)
			{
				// walk the responder chain until we get to a VC
				UIResponder *Responder = [[IOSAppDelegate GetDelegate].Window viewWithTag : 200];
				while (Responder != nil && ![Responder isKindOfClass:[UIViewController class]])
				{
					Responder = [Responder nextResponder];
				}
				ViewController = (UIViewController*)Responder;
			}
		}
	}
	
	if ([UIAlertController class])
	{
		UIAlertController* AlertController = [UIAlertController alertControllerWithTitle:[StringArray objectAtIndex : 0]
											message : [StringArray objectAtIndex : 1]
											preferredStyle:UIAlertControllerStyleAlert];

		// add any extra buttons
		for (int OptionalButtonIndex = 2; OptionalButtonIndex < [StringArray count]; OptionalButtonIndex++)
		{
			UIAlertAction* alertAction = [UIAlertAction
											actionWithTitle : [StringArray objectAtIndex : OptionalButtonIndex]
											style : UIAlertActionStyleDefault
											handler : ^ (UIAlertAction* action)
											{
												// just set our AlertResponse property, all we need to do
												self.AlertResponse = OptionalButtonIndex - 2;
												[AlertController dismissViewControllerAnimated : YES completion : nil];
											}
			];
			[AlertController addAction : alertAction];
		}

		NSLog(@"Showing Alert:\n  Title:%@\n  Message:%@", [StringArray objectAtIndex:0], [StringArray objectAtIndex:1]);
		if(ViewController != nil)
		{
			[ViewController presentViewController : AlertController animated : YES completion : nil];
		}
		else
		{
			NSLog(@"No valid ViewController to show Alert");
		}
	}
}

- (BOOL)textFieldShouldReturn:(UITextField *)alertTextField
{
	[alertTextField resignFirstResponder];// to dismiss the keyboard.
    
	return YES;
}

#if !UE_BUILD_SHIPPING && !PLATFORM_TVOS
- (void)SwipeLeftAction:(id)Ignored
{
	// Populate the text field with the previous entry in the history array
	if( self.ConsoleHistoryValues.count > 0 &&
		self.ConsoleHistoryValuesIndex + 1 < self.ConsoleHistoryValues.count )
	{
		self.ConsoleHistoryValuesIndex++;
        UITextField* TextField = nil;
        if ([UIAlertController class])
        {
            TextField = self.ConsoleAlertController.textFields.firstObject;
			TextField.text = [self.ConsoleHistoryValues objectAtIndex : self.ConsoleHistoryValuesIndex];
		}
	}
}

- (void)SwipeRightAction:(id)Ignored
{
	// Populate the text field with the next entry in the history array
	if( self.ConsoleHistoryValues.count > 0 &&
		self.ConsoleHistoryValuesIndex > 0 )
	{
		self.ConsoleHistoryValuesIndex--;
        UITextField* TextField = nil;
        if ([UIAlertController class])
        {
            TextField = self.ConsoleAlertController.textFields.firstObject;
			TextField.text = [self.ConsoleHistoryValues objectAtIndex : self.ConsoleHistoryValuesIndex];
        }
	}
}

#endif // !UE_BUILD_SHIPPING && !TVOS

void EnqueueConsoleCommand(uint8 *Command)
{
#if !UE_BUILD_SHIPPING
	if (!Command)
	{
		return;
	}

	[[IOSAppDelegate GetDelegate] HandleConsoleCommand:[NSString stringWithUTF8String : (const char *)Command]];
#endif
}

@end
