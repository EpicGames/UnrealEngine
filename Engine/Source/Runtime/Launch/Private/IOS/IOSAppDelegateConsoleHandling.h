// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOS/IOSAppDelegate.h"

@interface IOSAppDelegate (ConsoleHandling)

#if !UE_BUILD_SHIPPING && !PLATFORM_TVOS
/**
 * Shows the console input dialog
 */
- (void)ShowConsole;
#endif

/**
 * Shows an alert with up to 3 buttons. A delegate callback will later set AlertResponse property
 */
- (void)ShowAlert:(NSMutableArray*)StringArray;

#if !UE_BUILD_SHIPPING && !PLATFORM_TVOS
/**
 * Handles processing of an input console command
 */
- (void)HandleConsoleCommand:(NSString*)ConsoleCommand;
#endif


@end
