// Copyright Epic Games, Inc. All Rights Reserved.

#import <UIKit/UIKit.h>

#include "CoreMinimal.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceError.h"
#include "LaunchEngineLoop.h"
#include "IMessagingModule.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSApplication.h"
#include "IOS/IOSView.h"
#include "IOS/IOSCommandLineHelper.h"
#include "GameLaunchDaemonMessageHandler.h"
#include "AudioDevice.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "IOSAudioDevice.h"
#include "AudioMixerPlatformAudioUnitUtils.h"
#include "LocalNotification.h"
#include "Modules/ModuleManager.h"
#include "RenderingThread.h"
#include "GenericPlatform/GenericApplication.h"
#include "Misc/ConfigCacheIni.h"
#include "MoviePlayer.h"
#include "Containers/Ticker.h"
#include "HAL/ThreadManager.h"
#include "PreLoadScreenManager.h"
#include "TcpConsoleListener.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/EmbeddedCommunication.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FeedbackContext.h"
#include "Stats/ThermalStats.h"
#include "IOS/IOSPlatformCrashContext.h"
#include "ProfilingDebugging/CsvProfiler.h"

#import <Foundation/Foundation.h>
#import <UserNotifications/UNUserNotificationCenter.h>

#if PLATFORM_VISIONOS && UE_USE_SWIFT_UI_MAIN
#include "../Apple/SwiftMainBridge.h"
#endif

FEngineLoop GEngineLoop;
FGameLaunchDaemonMessageHandler GCommandSystem;

static FEvent* GSwiftTriggerEvent = FPlatformProcess::GetSynchEventFromPool();

static int32 DisableAudioSuspendOnAudioInterruptCvar = 1;
FAutoConsoleVariableRef CVarDisableAudioSuspendOnAudioInterrupt(
    TEXT("au.DisableAudioSuspendOnAudioInterrupt"),
    DisableAudioSuspendOnAudioInterruptCvar,
    TEXT("Disables callback for suspending the audio device when we are notified that the audio session has been interrupted.\n")
    TEXT("0: Not Disabled, 1: Disabled"),
    ECVF_Default);

static const double cMaxAudioContextResumeDelay = 0.5;    // Setting this to be 0.5 seconds
static double AudioContextResumeTime = 0;

void FAppEntry::ResetAudioContextResumeTime()
{
	AudioContextResumeTime = 0;
}

void FAppEntry::Suspend(bool bIsInterrupt)
{
	// also treats interrupts BEFORE initializing the engine
	// the movie player gets initialized on the preinit phase, ApplicationHasEnteredForegroundDelegate and ApplicationWillEnterBackgroundDelegate are not yet available
	if (GetMoviePlayer())
	{
		GetMoviePlayer()->Suspend();
	}
	
	FPreLoadScreenManager::EnableRendering(false);

	// if background audio is active, then we don't want to do suspend any audio
	if ([[IOSAppDelegate GetDelegate] IsFeatureActive:EAudioFeature::BackgroundAudio] == false)
	{
		if (GEngine && GEngine->GetMainAudioDevice() && !IsEngineExitRequested())
		{
			FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
			if (bIsInterrupt && DisableAudioSuspendOnAudioInterruptCvar)
			{
				if (FTaskGraphInterface::IsRunning() && !IsEngineExitRequested())
				{
					FFunctionGraphTask::CreateAndDispatchWhenReady([]()
					{
						FAudioThread::RunCommandOnAudioThread([]()
						{
							if (GEngine && GEngine->GetMainAudioDevice())
							{
								GEngine->GetMainAudioDevice()->SetTransientPrimaryVolume(0.0f);
							}
						}, TStatId());
					}, TStatId(), NULL, ENamedThreads::GameThread);
				}
				else
				{
					AudioDevice->SetTransientPrimaryVolume(0.0f);
				}
			}
			else
			{
				if (AudioContextResumeTime == 0)
				{
					// wait 0.5 sec before restarting the audio on resume
					// another Suspend event may occur when pulling down the notification center (Suspend-Resume-Suspend)
					AudioContextResumeTime = FPlatformTime::Seconds() + cMaxAudioContextResumeDelay;
				}
				else
				{
					//second resume, restart the audio immediately after resume
					AudioContextResumeTime = 0; 
				}

				if (FTaskGraphInterface::IsRunning())
				{
					FGraphEventRef ResignTask = FFunctionGraphTask::CreateAndDispatchWhenReady([]()
					{
						FAudioThread::RunCommandOnAudioThread([]()
						{
							if (GEngine && GEngine->GetMainAudioDevice())
							{
								GEngine->GetMainAudioDevice()->SuspendContext();
							}
						}, TStatId());
                
						FAudioCommandFence AudioCommandFence;
						AudioCommandFence.BeginFence();
						AudioCommandFence.Wait();
					}, TStatId(), NULL, ENamedThreads::GameThread);
					
					float BlockTime = [[IOSAppDelegate GetDelegate] GetBackgroundingMainThreadBlockTime];

					// Do not wait forever for this task to complete since the game thread may be stuck on waiting for user input from a modal dialog box
					FEmbeddedCommunication::KeepAwake(TEXT("Background"), false);
					double    startTime = FPlatformTime::Seconds();
					while((FPlatformTime::Seconds() - startTime) < BlockTime)
					{
						FPlatformProcess::Sleep(0.05f);
						if(ResignTask->IsComplete())
						{
							break;
						}
					}
					FEmbeddedCommunication::AllowSleep(TEXT("Background"));
				}
				else
				{
					AudioDevice->SuspendContext();
				}
			}
		}
		else
		{
            // Increment
            IncrementAudioSuspendCounters();
		}
	}
	
	if (FTaskGraphInterface::IsRunning())
	{
		FGraphEventRef FlushRenderThreadTask = FFunctionGraphTask::CreateAndDispatchWhenReady([]()
		{
			FlushRenderingCommands();
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}

void FAppEntry::Resume(bool bIsInterrupt)
{
	if (GetMoviePlayer())
	{
		GetMoviePlayer()->Resume();
	}
	
	FPreLoadScreenManager::EnableRendering(true);

	// if background audio is active, then we don't want to do suspend any audio
	// @todo: should this check if we were suspended, in case this changes while in the background? (suspend with background off, but resume with background audio on? is that a thing?)
	if ([[IOSAppDelegate GetDelegate] IsFeatureActive:EAudioFeature::BackgroundAudio] == false)
	{
		if (GEngine && GEngine->GetMainAudioDevice())
		{
			FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
        
			if (bIsInterrupt && DisableAudioSuspendOnAudioInterruptCvar)
			{
				if (FTaskGraphInterface::IsRunning())
				{
					FFunctionGraphTask::CreateAndDispatchWhenReady([]()
					{
						FAudioThread::RunCommandOnAudioThread([]()
						{
							if (GEngine && GEngine->GetMainAudioDevice())
							{
								GEngine->GetMainAudioDevice()->SetTransientPrimaryVolume(1.0f);
							}
						}, TStatId());
					}, TStatId(), NULL, ENamedThreads::GameThread);
				}
				else
				{
					AudioDevice->SetTransientPrimaryVolume(1.0f);
				}
			}
			else
			{
				if (AudioContextResumeTime != 0)
				{
					// resume audio on Tick()
					AudioContextResumeTime = FPlatformTime::Seconds() + cMaxAudioContextResumeDelay;
				}
				else
				{
					// resume audio immediately
					ResumeAudioContext();
				}
			}
		}
		else
		{
            // Decrement
            DecrementAudioSuspendCounters();
		}
	}
}


void FAppEntry::ResumeAudioContext()
{
	if (GEngine && GEngine->GetMainAudioDevice())
	{
		FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
		if (AudioDevice)
		{
			if (FTaskGraphInterface::IsRunning())
			{
				FFunctionGraphTask::CreateAndDispatchWhenReady([]()
				{
					FAudioThread::RunCommandOnAudioThread([]()
					{
						if (GEngine && GEngine->GetMainAudioDevice())
						{
							GEngine->GetMainAudioDevice()->ResumeContext();
						}
					}, TStatId());
				}, TStatId(), NULL, ENamedThreads::GameThread);
			}
			else
			{
				AudioDevice->ResumeContext();
			}
		}
	}
}

void FAppEntry::RestartAudio()
{
	if (GEngine && GEngine->GetMainAudioDevice())
	{
		FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();

		if (FTaskGraphInterface::IsRunning())
		{
            //increment the counter, otherwise ResumeContext won't work
            IncrementAudioSuspendCounters();

			FFunctionGraphTask::CreateAndDispatchWhenReady([]()
			{
				FAudioThread::RunCommandOnAudioThread([]()
				{
					if (GEngine && GEngine->GetMainAudioDevice())
					{
						GEngine->GetMainAudioDevice()->ResumeContext();
					}
				}, TStatId());
			}, TStatId(), NULL, ENamedThreads::GameThread);
		}
		else
		{
			AudioDevice->ResumeContext();
		}
	}
}

void FAppEntry::IncrementAudioSuspendCounters()
{
    // old backend
    if(FModuleManager::Get().IsModuleLoaded("IOSAudio"))
    {
        FIOSAudioDevice::IncrementSuspendCounter();
    }
    
    // new backend
    if(FModuleManager::Get().IsModuleLoaded("AudioMixerAudioUnit"))
    {
        Audio::IncrementIOSAudioMixerPlatformSuspendCounter();
    }
}

void FAppEntry::DecrementAudioSuspendCounters()
{
    // old backend
    if(FModuleManager::Get().IsModuleLoaded("IOSAudio"))
    {
        FIOSAudioDevice::DecrementSuspendCounter();
    }
    
    // new backend
    if(FModuleManager::Get().IsModuleLoaded("AudioMixerAudioUnit"))
    {
        Audio::DecrementIOSAudioMixerPlatformSuspendCounter();
    }
}

void FAppEntry::PreInit(IOSAppDelegate* AppDelegate, UIApplication* Application)
{
	// SwiftUI apps handle this differently
#if !UE_USE_SWIFT_UI_MAIN
	// make a controller object
	IOSViewController* IOSController = [[IOSViewController alloc] init];
	
	// IOSViewController is a GCViewController so we can have proper GC Controller support
	// But default we want all inputs to be forwaded to UI Kit. This will only change when we have physical keyborad or mouse connected
	IOSController.controllerUserInteractionEnabled = true;
	
#if PLATFORM_TVOS
	// @todo tvos: This may need to be exposed to the game so that when you click Menu it will background the app
	// this is basically the same way Android handles the Back button (maybe we should pass Menu button as back... maybe)
	IOSController.controllerUserInteractionEnabled = NO;
#endif
	
	// point to the GL view we want to use
	AppDelegate.RootView = [IOSController view];

	[AppDelegate.Window setRootViewController:IOSController];

#if !PLATFORM_TVOS
	// reset badge count on launch
	// Deprecated iOS17 - suppress warning if deployment target	> iOS17
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wdeprecated-declarations"
	UNUserNotificationCenter* currentNotificationCenter = [UNUserNotificationCenter currentNotificationCenter];
	if([currentNotificationCenter respondsToSelector:@selector(setBadgeCount:withCompletionHandler:)])
	{
		[currentNotificationCenter setBadgeCount:0 withCompletionHandler:nil];
	}
	else
	{
		// Setting applicationIconBadgeNumber to 0 also deletes all pending notifications from the Notification Center. Setting it to -1 
		// clears the badge count but doesn't delete pending notifications. This is an unofficial workaround and behaviour isn't documented.
		Application.applicationIconBadgeNumber = -1;
	}
#pragma clang diagnostic pop
#endif
	
#endif
}

static void MainThreadInit()
{
	FTaskTagScope::SetTagNone();
	
	// SwiftUI apps handle this differently
#if !UE_USE_SWIFT_UI_MAIN
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];

#if PLATFORM_VISIONOS
	CGRect MainFrame = CGRectMake(0, 0, 1000, 1000);
#else
	// Size the view appropriately for any potentially dynamically attached displays,
	// prior to creating any framebuffers
	CGRect MainFrame = [[UIScreen mainScreen] bounds];

#if UE_IOS_SCENE_LIFECYCLE && !PLATFORM_TVOS
	// In scene-based apps, +[UIScreen mainScreen].bounds is unreliable - it can keep
	// reporting portrait dimensions even when the scene is landscape. The scene's own
	// coordinateSpace.bounds is documented to always reflect the current orientation, so
	// query it directly. 
	if (UIWindowScene* WindowScene = AppDelegate.Window.windowScene)
	{
		AppDelegate.InterfaceOrientation = [WindowScene interfaceOrientation];
		AppDelegate.bDeviceInPortraitMode = UIInterfaceOrientationIsPortrait(AppDelegate.InterfaceOrientation);

		const CGRect SceneBounds = WindowScene.coordinateSpace.bounds;
		if (SceneBounds.size.width > 0 && SceneBounds.size.height > 0)
		{
			MainFrame = SceneBounds;
		}
	}
	
	// Seed FIOSApplication::CachedOrientation so the upcoming RebuildDisplayMetrics call
	// in FAppEntry::PlatformInit (which runs before the post-init didRotate: dispatch on
	// the main queue) doesn't transpose GetScreenRect into portrait. Without this, the
	// system resolution change request is fired at portrait dims, the MetalViewport is
	// created portrait, and the Metal drawable is locked to portrait until the user
	// triggers an orientation change.
	if (AppDelegate.InterfaceOrientation != UIInterfaceOrientationUnknown)
	{
		FIOSApplication::CachedOrientation = AppDelegate.InterfaceOrientation;
	}
#endif
#endif

	// @todo: use code similar for presizing for secondary screens
// 	CGRect FullResolutionRect =
// 		CGRectMake(
// 		0.0f,
// 		0.0f,
// 		GSystemSettings.bAllowSecondaryDisplays ?
// 		Max<float>(MainFrame.size.width, GSystemSettings.SecondaryDisplayMaximumWidth)	:
// 		MainFrame.size.width,
// 		GSystemSettings.bAllowSecondaryDisplays ?
// 		Max<float>(MainFrame.size.height, GSystemSettings.SecondaryDisplayMaximumHeight) :
// 		MainFrame.size.height
// 		);

	CGRect FullResolutionRect = MainFrame;

	// embedded apps are embedded inside a UE view, so it's already made
#if BUILD_EMBEDDED_APP
	// tell the embedded app that the .ini files are ready to be used, ie the View can be made if it was waiting to create the view
	FEmbeddedCallParamsHelper Helper;
	Helper.Command = TEXT("inisareready");
	FEmbeddedDelegates::GetEmbeddedToNativeParamsDelegateForSubsystem(TEXT("native")).Broadcast(Helper);
	// checkf(AppDelegate.IOSView != nil, TEXT("For embedded apps, the UEEmbeddedView must have been created and set into the AppDelegate as IOSView"));
#else
	AppDelegate.IOSView = [[FIOSView alloc] initWithFrame:FullResolutionRect];
	AppDelegate.IOSView.clearsContextBeforeDrawing = NO;
	
	
#if !PLATFORM_TVOS
	AppDelegate.IOSView.multipleTouchEnabled = YES;
#endif

	// add it to the window
	[AppDelegate.RootView addSubview:AppDelegate.IOSView];

	// initialize the backbuffer of the view (so the RHI can use it)
	[AppDelegate.IOSView CreateFramebuffer];
#endif
#endif
}


bool FAppEntry::IsStartupMoviePlaying()
{
	return GEngine && GEngine->IsInitialized() && GetMoviePlayer() && GetMoviePlayer()->IsStartupMoviePlaying();
}


void FAppEntry::PlatformInit()
{

	// call a function in the main thread to do some processing that needs to happen there, now that the .ini files are loaded
	dispatch_async(dispatch_get_main_queue(), ^{ MainThreadInit(); });

	// wait until the GLView is fully initialized, so the RHI can be initialized
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	
#if UE_USE_SWIFT_UI_MAIN
#if PLATFORM_VISIONOS
	// Read Immersive space settings and send them to swift where we will open the immersive space.
	int ImmersiveStyle = 0;
	GConfig->GetInt(TEXT("/Script/VisionOSRuntimeSettings.VisionOSRuntimeSettings"), TEXT("ImmersiveStyle"), ImmersiveStyle, GEngineIni);
	int UpperLimbVisibility = 2;
	GConfig->GetInt(TEXT("/Script/VisionOSRuntimeSettings.VisionOSRuntimeSettings"), TEXT("UpperLimbVisibility"), UpperLimbVisibility, GEngineIni);
	UE::SwiftMainBridgeNS::ConfigureImmersiveSpace(ImmersiveStyle, UpperLimbVisibility);
#endif //PLATFORM_VISIONOS
	GSwiftTriggerEvent->Wait();
#else
	
	while (!AppDelegate.IOSView || !AppDelegate.IOSView->bIsInitialized)
	{
#if BUILD_EMBEDDED_APP
		// while embedded, the native app may be waiting on some processing to happen before showing the view, so we have to let
		// processing occur here
		FTSTicker::GetCoreTicker().Tick(0.005f);
		FThreadManager::Get().Tick();
#endif //BUILD_EMBEDDED_APP
		FPlatformProcess::Sleep(0.005f);
	}
#endif //UE_USE_SWIFT_UI_MAIN
	// Set GSystemResolution now that we have the size.
	FDisplayMetrics DisplayMetrics;
	FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
	FSystemResolution::RequestResolutionChange(DisplayMetrics.PrimaryDisplayWidth, DisplayMetrics.PrimaryDisplayHeight, EWindowMode::Fullscreen);
	IConsoleManager::Get().CallAllConsoleVariableSinks();

	dispatch_async(dispatch_get_main_queue(), ^{
		AppDelegate.bPlatformInit = true;
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
		// Update rotation state now the game thread and taskgraph are ready
		[AppDelegate didRotate:nil];
#endif
	});
}

extern TcpConsoleListener *ConsoleListener;

static void ApplyIOSCompatConfigRules()
{
	TArray<FString> IOSCompatCVars;
	if (GConfig->GetArray(TEXT("IOSCompatCVars"), TEXT("CVars"), IOSCompatCVars, GEngineIni))
	{		
		TSet<FString> AllowedCompatCVars(IOSCompatCVars);
		for (const TTuple<FString, FString>& Pair : FPlatformMisc::GetConfigRuleVars())
		{
			const FString& Key = Pair.Key;
			const FString& Value = Pair.Value;
			static const TCHAR IOSCompat[] = TEXT("IOSCompat.");
			if (Key.StartsWith(IOSCompat))
			{
				FString CVarName = Key.Mid(UE_ARRAY_COUNT(IOSCompat) - 1);
				if (AllowedCompatCVars.Contains(CVarName))
				{
					auto* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
					if (CVar)
					{
						// set with HF priority
						// configrules are therefore higher priority than deviceprofiles. (e.g. -dpcvars)
						CVar->Set(*Value, ECVF_SetByHotfix);
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Config rule compat Setting %s = %s"), *CVarName, *Value);
					}
				}
			}
		}
	}
}

static bool ShouldUseLoggingThread()
{
	bool bAllowLoggingThread = false;
	GConfig->GetBool(FPlatformProperties::GetRuntimeSettingsClassName(), TEXT("bAllowLoggingThread"), bAllowLoggingThread, GEngineIni);

	const FString* ConfigRulesOverride = FPlatformMisc::GetConfigRulesVariable(TEXT("AllowLoggingThread"));
	if (ConfigRulesOverride)
	{
		bAllowLoggingThread = ConfigRulesOverride->Equals("true", ESearchCase::IgnoreCase);
	}

	return bAllowLoggingThread;
}

void FAppEntry::Init()
{
	SCOPED_BOOT_TIMING("FAppEntry::Init()");
	
	FDelegateHandle ConfigReadyHandle = FCoreDelegates::TSConfigReadyForUse().AddStatic(&ApplyIOSCompatConfigRules);

	//extern TCHAR GCmdLine[16384];
	GEngineLoop.PreInit(FCommandLine::Get());

	// initialize messaging subsystem
	FModuleManager::LoadModuleChecked<IMessagingModule>("Messaging");

	//Set up the message handling to interface with other endpoints on our end.
	NSLog(@"%s", "Initializing ULD Communications in game mode\n");
	GCommandSystem.Init();

    // Detect low power mode 
    bool bLowPower = [[NSProcessInfo processInfo] isLowPowerModeEnabled];
	NSLog(@"Low power mode: %s", bLowPower ? "ENABLED" : "DISABLED");

	if (!ShouldUseLoggingThread())
	{
		GLog->SetCurrentThreadAsPrimaryThread();
	}

	// Send the launch local notification to the local notification service now that the engine module system has been initialized
	if(gAppLaunchedWithLocalNotification)
	{
		ILocalNotificationService* notificationService = NULL;

		// Get the module name from the .ini file
		FString ModuleName;
		GConfig->GetString(TEXT("LocalNotification"), TEXT("DefaultPlatformService"), ModuleName, GEngineIni);

		if (ModuleName.Len() > 0)
		{			
			// load the module by name retrieved from the .ini
			ILocalNotificationModule* module = FModuleManager::LoadModulePtr<ILocalNotificationModule>(*ModuleName);

			// does the module exist?
			if (module != nullptr)
			{
				notificationService = module->GetLocalNotificationService();
				if(notificationService != NULL)
				{
					notificationService->SetLaunchNotification(gLaunchLocalNotificationActivationEvent, gLaunchLocalNotificationFireDate);
				}
			}
		}
	}

	// start up the engine
	GEngineLoop.Init();

#if CSV_PROFILER
    // Add CSV metadata indicating if we're in low power mode at the end of each CSV capture
	FCsvProfiler::Get()->OnCSVProfileEndRequested().AddStatic([]() { 
		bool bLowPower = [[NSProcessInfo processInfo]isLowPowerModeEnabled]; 
		CSV_METADATA(TEXT("IOS/LowPowerMode"), bLowPower ? TEXT("1") : TEXT("0"));
	});
#endif

#if !UE_BUILD_SHIPPING
	FIPv4Endpoint ConsoleTCP(FIPv4Address::InternalLoopback, 8888); //TODO: read this from an .ini
	if (ConsoleListener == nullptr)
	{
		ConsoleListener = new TcpConsoleListener(ConsoleTCP);
	}
	// tear down the console listener when backgrounded
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddLambda([]()
	{
		if (ConsoleListener)
		{
			delete ConsoleListener;
			ConsoleListener = nullptr;
		}
	});
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddLambda([ConsoleTCP]()
	{
		if (ConsoleListener == nullptr)
		{
			ConsoleListener = new TcpConsoleListener(ConsoleTCP);
		}
	});



#endif // UE_BUILD_SHIPPING
}

#if BUILD_EMBEDDED_APP
static bool GWasTickSuspended = false;
static double GPreviousSuspendTime = FPlatformTime::Seconds();
#endif

void FAppEntry::Tick()
{
#if BUILD_EMBEDDED_APP
	GWasTickSuspended = false;
#endif
	UE::ThermalStats::GetAndPushStats();
    
	if (AudioContextResumeTime != 0)
	{
		if (FPlatformTime::Seconds() >= AudioContextResumeTime)
		{
			ResumeAudioContext();
			AudioContextResumeTime = 0;
		}
	}

	// tick the engine
	GEngineLoop.Tick();
}

void FAppEntry::SuspendTick()
{
#if BUILD_EMBEDDED_APP
	static double PreviousTime = FPlatformTime::Seconds();
    if (!GWasTickSuspended)
    {
		GWasTickSuspended = true;
		// reset it each time we background
		PreviousTime = FPlatformTime::Seconds();
    }

	float DeltaTime = FPlatformTime::Seconds() - PreviousTime;
	PreviousTime = FPlatformTime::Seconds();

	// allow for some background processing
	FEmbeddedCommunication::TickGameThread(DeltaTime);
	FCoreDelegates::MobileBackgroundTickDelegate.Broadcast(DeltaTime);
#endif

	FPlatformProcess::Sleep(0.1f);
}

void FAppEntry::Shutdown()
{
	if (ConsoleListener)
	{
		delete ConsoleListener;
	}
	NSLog(@"%s", "Shutting down Game ULD Communications\n");
	GCommandSystem.Shutdown();
    
    // kill the engine
    GEngineLoop.Exit();
}

bool	FAppEntry::gAppLaunchedWithLocalNotification;
FString	FAppEntry::gLaunchLocalNotificationActivationEvent;
int32	FAppEntry::gLaunchLocalNotificationFireDate;

FString GSavedCommandLine;

void EngineCrashHandler(const FGenericCrashContext& GenericContext)
{
	const FIOSCrashContext& Context = static_cast<const FIOSCrashContext&>(GenericContext);
	Context.ReportCrash();
	
	if (GLog)
	{
		GLog->Panic();
	}
	
	if (GWarn)
	{
		GWarn->Flush();
	}
	
	if (GError)
	{
		GError->Flush();
		GError->HandleError();
	}
	
	return Context.GenerateCrashInfo();
}

#if !BUILD_EMBEDDED_APP && !UE_USE_SWIFT_UI_MAIN
int main(int argc, char *argv[])
{
    for(int Option = 1; Option < argc; Option++)
	{
		GSavedCommandLine += TEXT(" ");
		GSavedCommandLine += UTF8_TO_TCHAR(argv[Option]);
	}

	// convert $'s to " because Xcode swallows the " and this will allow -execcmds= to be usable from xcode
	GSavedCommandLine = GSavedCommandLine.Replace(TEXT("$"), TEXT("\""));

	FIOSCommandLineHelper::InitCommandArgs(FString());
	
#if !UE_BUILD_SHIPPING
    if (FParse::Param(FCommandLine::Get(), TEXT("WaitForDebugger")))
    {
        while(!FPlatformMisc::IsDebuggerPresent())
        {
            FPlatformMisc::LowLevelOutputDebugString(TEXT("Waiting for debugger...\n"));
            FPlatformProcess::Sleep(1.f);
        }
        FPlatformMisc::LowLevelOutputDebugString(TEXT("Debugger attached.\n"));
    }
#endif
	
#if !UE_ENABLE_IOS_CRASHDUMP
	if (FParse::Param(FCommandLine::Get(), TEXT("EnableIOSCrashDump")))
#endif
	{
		FPlatformMisc::SetCrashHandler(EngineCrashHandler);
	}
    
	@autoreleasepool {
	    return UIApplicationMain(argc, argv, nil, NSStringFromClass([IOSAppDelegate class]));
	}
}

#endif


#include "Apple/LaunchCPPToSwift.h"

#if PLATFORM_VISIONOS

void KickoffWithCompositingLayer(CP_OBJECT_cp_layer_renderer* Layer)
{
#if UE_USE_SWIFT_UI_MAIN
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
    
    // Might need to cp_layer_renderer_configuration_set_layout here,  or in UESwift... in the future.
	
	cp_layer_renderer_properties_t Props = cp_layer_renderer_get_properties(Layer);
	int NumViews = cp_layer_renderer_properties_get_view_count(Props);

	NSMutableArray* Viewports = [NSMutableArray arrayWithCapacity:NumViews];

	// get the texture topology
	// @todo when Apple adds the API to actually get the size, use this instead of the mess below (docs indicate you can get
	// get the width/height, but there's no functions to get them 
	//int NumTopologies = cp_layer_renderer_properties_get_texture_topology_count(Props);
//	for (int TopoIndex = 0; TopoIndex < NumToplogies; TopoIndex++)
//	{
//		cp_texture_topology_t Topology = cp_layer_renderer_properties_get_texture_topology(Props, TopoIndex);
//	
////		NSValue* VPValue = [NSValue valueWithCGRect:CGRectMake(Viewport.originX, Viewport.originY, Viewport.width, Viewport.height)];
////		[Viewports addObject:VPValue];
//	}

	{
		cp_frame_t SwiftLayerFrame = cp_layer_renderer_query_next_frame(Layer);
		cp_drawable_t SwiftDrawable = cp_frame_query_drawable(SwiftLayerFrame);
		for (int ViewIndex = 0; ViewIndex < NumViews; ViewIndex++)
		{
			cp_view_t View = cp_drawable_get_view(SwiftDrawable, ViewIndex);
			cp_view_texture_map_t TextureMap = cp_view_get_view_texture_map(View);
			MTLViewport Viewport = cp_view_texture_map_get_viewport(TextureMap);

			float X = Viewport.originX;
			float Y = Viewport.originY;
			float W = Viewport.width;
			float H = Viewport.height;
			NSValue* VPValue = [NSValue valueWithCGRect:CGRectMake(X, Y, W, H)];
			[Viewports addObject:VPValue];
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Adding eye viewport : [%f. %f] / [%f x %f]\n"), X, Y, W, H);
		}
		cp_frame_start_submission(SwiftLayerFrame);
		id<MTLDevice> Device = cp_layer_renderer_get_device(Layer);
		id<MTLCommandQueue> CommandQueue = [[Device newCommandQueue] autorelease];
		id<MTLCommandBuffer> CommandBuffer = [CommandQueue commandBuffer];
		cp_drawable_encode_present(cp_frame_query_drawable(SwiftLayerFrame), CommandBuffer);
		[CommandBuffer commit];
		cp_frame_end_submission(SwiftLayerFrame);
	}

	// cache the viewports in the delegate so code later can get it when asking about the screen bounds
	AppDelegate.SwiftLayerViewports = Viewports;
	
	CGRect FirstViewport = [[AppDelegate.SwiftLayerViewports firstObject] CGRectValue];
	
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Kicking off UE with Swift Layer. Commandline: %s\n"), *GSavedCommandLine);
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("NumViews: %d, Full size = %f x %f\n"), NumViews, FirstViewport.size.width, FirstViewport.size.height);

	AppDelegate.SwiftLayer = Layer;
	
	// unleash the bootup process
	GSwiftTriggerEvent->Trigger();
#endif
}

#else

// swift may link to this even if we aren't using a Swift main
void KickoffEngine()
{
#if UE_USE_SWIFT_UI_MAIN
	FTaskTagScope::SetTagNone();

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Kicking off UE from Swift. Commandline: %s\n"), *GSavedCommandLine);
	GSwiftTriggerEvent->Trigger();
#endif	
}

#endif
