// Copyright Epic Games, Inc. All Rights Reserved.

#include "TcpConsoleListener.h"
#include "HAL/RunnableThread.h"
#include "Common/TcpSocketBuilder.h"
#include "Common/TcpListener.h"

extern void EnqueueConsoleCommand(uint8* Command);
TcpConsoleListener* ConsoleListener = nullptr;

/* TcpConsoleListener structors
 *****************************************************************************/

TcpConsoleListener::TcpConsoleListener(const FIPv4Endpoint& InListenEndpoint)
	: ListenEndpoint(InListenEndpoint)
	, bStopping(false)
	, Listener(nullptr)
{
	UE_LOGF(LogTemp, Display, "Initializing TCPConsoleListener.");
	Thread = FRunnableThread::Create(this, TEXT("TcpConsoleListener"), 128 * 1024, TPri_Normal);
}

TcpConsoleListener::~TcpConsoleListener()
{
	UE_LOGF(LogTemp, Display, "Stopping TCPConsoleListener.");
	if (Listener)
	{
		delete Listener;
		Listener = nullptr;
	}
	
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}
}

bool TcpConsoleListener::HandleListenerConnectionAccepted(FSocket* ClientSocket, const FIPv4Endpoint& ClientEndpoint)
{
	Connections.Add(ClientSocket);
	return true;
}

/* FRunnable interface
*****************************************************************************/

void TcpConsoleListener::Exit()
{
	// do nothing
}


bool TcpConsoleListener::Init()
{
	Listener = new FTcpListener(ListenEndpoint);
	Listener->OnConnectionAccepted().BindRaw(this, &TcpConsoleListener::HandleListenerConnectionAccepted);

	return true;
}


uint32 TcpConsoleListener::Run()
{
	static const SIZE_T CommandSize = 1024;
	uint8 RecvBuffer[CommandSize];
	/** Current connections */
	TArray<FSocket*> ToRemove;

	while (!bStopping)
	{
		ToRemove.Empty();

		for (auto Connection : Connections)
		{
			uint32 PendingDataSize;
			if (Connection->HasPendingData(PendingDataSize))
			{
				int32 BytesRead = 0;
				memset(RecvBuffer, 0, CommandSize);
				if (Connection->Recv(RecvBuffer, CommandSize, BytesRead))
				{
					// strip any carriage return/newline character from end of line
					int32 BufferIdx = BytesRead - 1;
					while (BufferIdx >= 0 && (RecvBuffer[BufferIdx] == '\n' || RecvBuffer[BufferIdx] == '\r'))
					{
						RecvBuffer[BufferIdx--] = 0;
					}
					UE_LOGF(LogTemp, Display, "Received TCP console command '%ls'", *FString(UTF8_TO_TCHAR(RecvBuffer)));
					EnqueueConsoleCommand(RecvBuffer);
				}
				else
				{
					ToRemove.Add(Connection);
				}
			}
		}

		for (auto& Remove : ToRemove)
		{
			Connections.Remove(Remove);
		}
		
		FPlatformProcess::Sleep(0.5f);
	}

	return 0;
}

void TcpConsoleListener::Stop()
{
	bStopping = true;

	if (Listener)
	{
		delete Listener;
		Listener = nullptr;
	}
}
