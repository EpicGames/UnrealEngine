// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

class FSocket;
class FTcpListener;

/**
 *
 */
class TcpConsoleListener
	: FRunnable
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 */
	TcpConsoleListener(const FIPv4Endpoint& InListenEndpoint);

	/** Virtual destructor. */
	virtual ~TcpConsoleListener();

public:

	//~ FRunnable interface

	virtual void Exit() override;
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;

private:
	
	/** Callback for accepted connections to the local server. */
	bool HandleListenerConnectionAccepted(FSocket* ClientSocket, const FIPv4Endpoint& ClientEndpoint);

	/** Current connections */
	TArray<FSocket*> Connections;

	/** Current settings */
	FIPv4Endpoint ListenEndpoint;

	/** For the thread */
	bool bStopping;

	/** Holds the local listener for incoming tunnel connections. */
	FTcpListener* Listener;

	/** Holds the thread object. */
	FRunnableThread* Thread;
};
