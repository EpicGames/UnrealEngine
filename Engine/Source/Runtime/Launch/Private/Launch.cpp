// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceError.h"
#include "LaunchEngineLoop.h"
#include "PhysicsPublic.h"
#include "HAL/ExceptionHandling.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Stats/StatsMisc.h"
#include "Misc/CoreDelegates.h"
#include "Misc/EngineVersion.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/TrackedActivity.h"
#if WITH_EDITOR
	#include "UnrealEdGlobals.h"
#endif
#if PLATFORM_WINDOWS
	#include "Windows/WindowsHWrapper.h"
#endif

IMPLEMENT_MODULE(FDefaultModuleImpl, Launch);

#if PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_UNIX || PLATFORM_USE_GENERIC_LAUNCH_IMPLEMENTATION

FEngineLoop	GEngineLoop;

extern "C" int test_main(int argc, char ** argp)
{
	return 0;
}

/** 
 * PreInits the engine loop 
 */
int32 EnginePreInit( const TCHAR* CmdLine )
{
	int32 ErrorLevel = GEngineLoop.PreInit( CmdLine );

	return( ErrorLevel );
}

/** 
 * Inits the engine loop 
 */
int32 EngineInit()
{
	int32 ErrorLevel = GEngineLoop.Init();

	return( ErrorLevel );
}

/** 
 * Ticks the engine loop 
 */
LAUNCH_API void EngineTick( void )
{
	GEngineLoop.Tick();
}

/**
 * Shuts down the engine
 */
LAUNCH_API void EngineExit( void )
{
	// Make sure this is set
	RequestEngineExit(TEXT("EngineExit() was called"));

	GEngineLoop.Exit();
}

/**
 * Performs any required cleanup in the case of a fatal error.
 */
void LaunchStaticShutdownAfterError()
{
	// Make sure physics is correctly torn down.
	TermGamePhys();
}

/**
 * Static guarded main function. Rolled into own function so we can have error handling for debug/ release builds depending
 * on whether a debugger is attached or not.
 */
int32 GuardedMain( const TCHAR* CmdLine )
{
	FTrackedActivity::GetEngineActivity().Update(TEXT("Starting"), FTrackedActivity::ELight::Yellow);

	FTaskTagScope Scope(ETaskTag::EGameThread);
	// remember thread id of the main thread
	GGameThreadId = FPlatformTLS::GetCurrentThreadId();
	GIsGameThreadIdInitialized = true;

#if !(UE_BUILD_SHIPPING)

	// If "-waitforattach" or "-WaitForDebugger" was specified, halt startup and wait for a debugger to attach before continuing

	const bool bWaitForDebuggerAndBreak = FParse::Param(CmdLine, TEXT("waitforattach")) || FParse::Param(CmdLine, TEXT("WaitForDebugger")); 
	const bool bWaitForDebugger = bWaitForDebuggerAndBreak || FParse::Param(CmdLine, TEXT("WaitForAttachNoBreak")) || FParse::Param(CmdLine, TEXT("WaitForDebuggerNoBreak"));
	
	if (bWaitForDebugger)
	{
		while (!FPlatformMisc::IsDebuggerPresent())
		{
			FPlatformProcess::Sleep(0.1f);
		}

		if (bWaitForDebuggerAndBreak)
		{
			UE_DEBUG_BREAK();
		}
	}

#endif

	BootTimingPoint("DefaultMain");

	// Super early init code. DO NOT MOVE THIS ANYWHERE ELSE!
	FCoreDelegates::GetPreMainInitDelegate().Broadcast();

	// make sure GEngineLoop::Exit() is always called.
	struct EngineLoopCleanupGuard 
	{ 
		~EngineLoopCleanupGuard()
		{
			// Don't shut down the engine on scope exit when we are running embedded
			// because the outer application will take care of that.
			if (!GUELibraryOverrideSettings.bIsEmbedded)
			{
				EngineExit();
			}
		}
	} CleanupGuard;

	// Set up minidump filename. We cannot do this directly inside main as we use an FString that requires 
	// destruction and main uses SEH.
	// These names will be updated as soon as the Filemanager is set up so we can write to the log file.
	// That will also use the user folder for installed builds so we don't write into program files or whatever.
#if PLATFORM_WINDOWS
	FCString::Strcpy(MiniDumpFilenameW, *FString::Printf(TEXT("unreal-v%i-%s.dmp"), FEngineVersion::Current().GetChangelist(), *FDateTime::Now().ToString()));
#endif

	FTrackedActivity::GetEngineActivity().Update(TEXT("Initializing"));
	int32 ErrorLevel = EnginePreInit( CmdLine );

	// exit if PreInit failed.
	if ( ErrorLevel != 0 || IsEngineExitRequested() )
	{
		return ErrorLevel;
	}

	{
		FScopedSlowTask SlowTask(100, NSLOCTEXT("EngineInit", "EngineInit_Loading", "Loading..."));

		// EnginePreInit leaves 20% unused in its slow task.
		// Here we consume 80% immediately so that the percentage value on the splash screen doesn't change from one slow task to the next.
		// (Note, we can't include the call to EnginePreInit in this ScopedSlowTask, because the engine isn't fully initialized at that point)
		SlowTask.EnterProgressFrame(80);

		SlowTask.EnterProgressFrame(20);

#if WITH_EDITOR
		if (GIsEditor)
		{
			ErrorLevel = EditorInit(GEngineLoop);
		}
		else
#endif
		{
			ErrorLevel = EngineInit();
		}
	}

	double EngineInitializationTime = FPlatformTime::Seconds() - GStartTime;
	UE_LOGF(LogLoad, Log, "(Engine Initialization) Total time: %.2f seconds", EngineInitializationTime);

	ACCUM_LOADTIME(TEXT("EngineInitialization"), EngineInitializationTime);

	BootTimingPoint("Tick loop starting");
	DumpBootTiming();

	FTrackedActivity::GetEngineActivity().Update(TEXT("Ticking loop"), FTrackedActivity::ELight::Green);

	// Don't tick if we're running an embedded engine - we rely on the outer
	// application ticking us instead.
	if (!GUELibraryOverrideSettings.bIsEmbedded)
	{
		while( !IsEngineExitRequested() )
		{
			EngineTick();
		}
	}

	TRACE_BOOKMARK(TEXT("Tick loop end"));

#if WITH_EDITOR
	if( GIsEditor )
	{
		EditorExit();
	}
#endif
	return ErrorLevel;
}

#endif
