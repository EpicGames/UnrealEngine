// Copyright Epic Games, Inc. All Rights Reserved.

#include "LaunchEngineLoop.h"

#include "AutoRTFM/AutoRTFMUE.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformOutputDevices.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MallocFrameProfiler.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/QueuedThreadPool.h"
#include "Misc/QueuedThreadPoolWrapper.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformAffinity.h"
#include "HAL/DiskUtilizationTracker.h"
#include "Misc/FileHelper.h"
#include "Internationalization/TextLocalizationManagerGlobals.h"
#include "Logging/LogSuppressionInterface.h"
#include "Logging/StructuredLog.h"
#include "Async/TaskGraphInterfaces.h"
#include "Async/Fundamental/Scheduler.h"
#include "MemPro/MemProProfiler.h"
#include "Misc/AsciiSet.h"
#include "Misc/TimeGuard.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreMisc.h"
#include "Misc/ConfigUtilities.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/AutomationTest.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/OutputDeviceStdOut.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Misc/TrackedActivity.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMemoryHelpers.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/IPlatformFileManagedStorageWrapper.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Stats/StatsSystem.h"
#include "Stats/ThreadIdleStats.h"
#include "Trace/Trace.inl"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/BootProfiling.h"
#if WITH_ENGINE
#include "HAL/PlatformSplash.h"
#include "RenderTimer.h"
#include "SceneInterface.h"
#include "StereoRenderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#endif
#if WITH_APPLICATION_CORE
#include "HAL/PlatformApplicationMisc.h"
#endif
#include "HAL/ThreadManager.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "ProfilingDebugging/StallDetector.h"
#include "Containers/StringView.h"
#include "Containers/Ticker.h"

#include "Interfaces/IPluginManager.h"
#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"
#include "Misc/UProjectInfo.h"
#include "Misc/EngineVersion.h"

#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "Runtime/Launch/Resources/Version.h"
#include "Modules/BuildVersion.h"
#include "UObject/DevObjectVersion.h"
#include "HAL/ThreadHeartBeat.h"

#include "Misc/NetworkVersion.h"
#include "Templates/UniquePtr.h"

#include "Compression/OodleDataCompression.h"
#include "Misc/Compression.h"

#if !(IS_PROGRAM || WITH_EDITOR)
#include "IPlatformFilePak.h"
#endif

#ifndef USE_IO_DISPATCHER
#define USE_IO_DISPATCHER (WITH_ENGINE || WITH_IOSTORE_IN_EDITOR || !(IS_PROGRAM || WITH_EDITOR))
#endif
#if USE_IO_DISPATCHER
#include "IO/IoDispatcher.h"
#endif

#if !defined(UE_WITH_IOSTOREONDEMAND)
#define UE_WITH_IOSTOREONDEMAND 0
#endif

#if WITH_COREUOBJECT
	#include "Internationalization/PackageLocalizationManager.h"
	#include "Misc/PackageName.h"
	#include "UObject/UObjectHash.h"
	#include "UObject/Package.h"
	#include "UObject/Linker.h"
	#include "UObject/LinkerLoad.h"
	#include "UObject/PackageResourceManager.h"
	#include "UObject/ReferencerFinder.h"

	#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		#include "VerseVM/VVMVerse.h"
		#include "VerseVM/VVMWriteBarrier.h"
	#else
		#include "VerseVM/VVMExecutionContext.h"
	#endif
#endif

#if WITH_EDITOR
	#include "Blueprint/BlueprintSupport.h"
#if WITH_ENGINE
	#include "Styling/AppStyle.h"
#endif// WITH_ENGINE
	#include "Misc/RemoteConfigIni.h"
#if WITH_ENGINE
	#include "EditorCommandLineUtils.h"
	#include "Input/Reply.h"
	#include "Styling/CoreStyle.h"
	#include "RenderingThread.h"
	#include "RenderDeferredCleanup.h"
	#include "Editor/EditorEngine.h"
	#include "UnrealEdMisc.h"
	#include "UnrealEdGlobals.h"
	#include "UObject/StrongObjectPtr.h"
	#include "Editor/UnrealEdEngine.h"
	#include "Settings/EditorExperimentalSettings.h"
#endif// WITH_ENGINE
	#include "Serialization/BulkDataRegistry.h"
	#include "Virtualization/VirtualizationSystem.h"
#if WITH_ENGINE
	#include "AnimationUtils.h"
#endif// WITH_ENGINE
	#include "UObject/ICookInfo.h"

	#if PLATFORM_WINDOWS
		#include "Windows/AllowWindowsPlatformTypes.h"
			#include <objbase.h>
		#include "Windows/HideWindowsPlatformTypes.h"
		#include "Windows/WindowsPlatformPerfCounters.h"
	#endif
#endif //WITH_EDITOR

#if WITH_ENGINE
	#include "AssetCompilingManager.h"
	#include "Engine/GameEngine.h"
	#include "Engine/GameViewportClient.h"
	#include "UnrealClient.h"
	#include "Engine/LocalPlayer.h"
	#include "GameFramework/PlayerController.h"
	#include "GameFramework/GameUserSettings.h"
	#include "Features/IModularFeatures.h"
	#include "GameFramework/WorldSettings.h"
	#include "SystemSettings.h"
	#include "EngineStats.h"
	#include "EngineGlobals.h"
	#include "AudioThread.h"
	#include "AudioDeviceManager.h"
#if !UE_BUILD_SHIPPING
	#include "IAutomationControllerModule.h"
#endif // !UE_BUILD_SHIPPING
#if WITH_EDITORONLY_DATA
	#include "DerivedDataBuild.h"
	#include "DerivedDataCache.h"
#endif // WITH_EDITORONLY_DATA
	#include "DerivedDataCacheInterface.h"
	#include "Serialization/DerivedData.h"
	#include "ShaderCompiler.h"
	#include "DistanceFieldAtlas.h"
	#include "MeshCardBuild.h"
	#include "GlobalShader.h"
	#include "ShaderCodeLibrary.h"
	#include "Materials/MaterialInterface.h"
	#include "TextureResource.h"
	#include "Engine/Texture2D.h"
	#include "Internationalization/StringTable.h"
	#include "SceneUtils.h"
	#include "ParticleHelper.h"
	#include "PhysicsPublic.h"
	#include "PlatformFeatures.h"
	#include "DeviceProfiles/DeviceProfileManager.h"
	#include "Commandlets/Commandlet.h"
	#include "EngineService.h"
	#include "TraceService.h"
	#include "ContentStreaming.h"
	#include "HighResScreenshot.h"
	#include "Misc/HotReloadInterface.h"
	#include "ISessionServicesModule.h"
	#include "Net/OnlineEngineInterface.h"
	#include "Internationalization/EnginePackageLocalizationCache.h"
	#include "Rendering/SlateRenderer.h"
	#include "Layout/WidgetPath.h"
	#include "Framework/Application/SlateApplication.h"
	#include "IMessagingModule.h"
	#include "Engine/DemoNetDriver.h"
	#include "LongGPUTask.h"
	#include "RenderUtils.h"
	#include "DynamicResolutionState.h"
	#include "EngineModule.h"
	#include "DumpGPU.h"
	#include "PSOPrecacheMaterial.h"
	#include "VisualizeTexture.h"

#if !UE_SERVER
	#include "AppMediaTimeSource.h"
	#include "IHeadMountedDisplayModule.h"
	#include "IMediaModule.h"
	#include "HeadMountedDisplay.h"
	#include "MRMeshModule.h"
	#include "Interfaces/ISlateRHIRendererModule.h"
	#include "Interfaces/ISlateNullRendererModule.h"
	#include "EngineFontServices.h"
#endif

	#include "MoviePlayer.h"
	#include "MoviePlayerProxy.h"
	#include "PreLoadScreenManager.h"
	#include "InstallBundleManagerInterface.h"

	#include "ShaderCodeLibrary.h"
	#include "ShaderPipelineCache.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#include "ProfileVisualizerModule.h"
#endif

#if WITH_AUTOMATION_WORKER
	#include "IAutomationWorkerModule.h"
#endif

#if WITH_ODSC
	#include "ODSC/ODSCManager.h"
#endif
#endif  //WITH_ENGINE

#include "Misc/EmbeddedCommunication.h"

#if WITH_ENGINE
	#include "Tests/RHIUnitTests.h"
#endif

class FSlateRenderer;
class SViewport;
class IPlatformFile;
class FExternalProfiler;
class FFeedbackContext;

#if WITH_EDITOR && WITH_ENGINE
	#include "FeedbackContextEditor.h"
	static FFeedbackContextEditor UnrealEdWarn;
	#include "AudioEditorModule.h"
#endif	// WITH_EDITOR

#if UE_EDITOR
	#include "DesktopPlatformModule.h"
	#include "TargetReceipt.h"
#endif

#define LOCTEXT_NAMESPACE "LaunchEngineLoop"

#if PLATFORM_WINDOWS
	#include "Windows/AllowWindowsPlatformTypes.h"
	#include <ObjBase.h>
	#include "Windows/HideWindowsPlatformTypes.h"
	#include <DbgHelp.h>
#endif

#if WITH_ENGINE
	#include "EngineDefines.h"
	#if ENABLE_VISUAL_LOG
		#include "VisualLogger/VisualLogger.h"
	#endif
	#include "FramePro/FrameProProfiler.h"
	#include "ProfilingDebugging/CsvProfiler.h"
#endif

#if defined(WITH_LAUNCHERCHECK) && WITH_LAUNCHERCHECK
	#include "ILauncherCheckModule.h"
#endif
#include "String/ParseTokens.h"

#if WITH_COREUOBJECT
	#ifndef USE_LOCALIZED_PACKAGE_CACHE
		#define USE_LOCALIZED_PACKAGE_CACHE 1
	#endif
#else
	#define USE_LOCALIZED_PACKAGE_CACHE 0
#endif

#if WITH_ENGINE
	CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);
#endif

#ifndef WITH_CONFIG_PATCHING
#define WITH_CONFIG_PATCHING 0
#endif

#if WITH_APPLICATION_CORE && (PLATFORM_IOS || PLATFORM_TVOS)
#include "IOS/IOSAppDelegate.h"
#endif

#ifndef UE_MERGED_MODULES
#define UE_MERGED_MODULES 0
#endif

#if CPUPROFILERTRACE_ENABLED
UE_TRACE_EVENT_BEGIN(Cpu, Frame, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_END()
#endif // CPUPROFILERTRACE_ENABLED

#if ENABLE_NAMED_EVENTS
// Different levels of color for scoped named events 
static const FColor FrameColor = FColor::Red;
static const FColor SystemColor = FColor::Orange;
static const FColor CoreDelegatesColor = FColor::Cyan;
#endif // ENABLE_NAMED_EVENTS

bool GIsConsoleExecutable = false;

static TAutoConsoleVariable<int32> CVarDoAsyncEndOfFrameTasksRandomize(
	TEXT("tick.DoAsyncEndOfFrameTasks.Randomize"),
	0,
	TEXT("Used to add random sleeps to tick.DoAsyncEndOfFrameTasks to shake loose bugs on either thread. Also does random render thread flushes from the game thread.")
	);

static TAutoConsoleVariable<int32> CVarDoAsyncEndOfFrameTasksValidateReplicatedProperties(
	TEXT("tick.DoAsyncEndOfFrameTasks.ValidateReplicatedProperties"),
	0,
	TEXT("If true, validates that replicated properties haven't changed during the Slate tick. Results will not be valid if demo.ClientRecordAsyncEndOfFrame is also enabled.")
	);

static FAutoConsoleTaskPriority CPrio_AsyncEndOfFrameGameTasks(
	TEXT("TaskGraph.TaskPriorities.AsyncEndOfFrameGameTasks"),
	TEXT("Task and thread priority for the experiemntal async end of frame tasks."),
	ENamedThreads::HighThreadPriority,
	ENamedThreads::NormalTaskPriority,
	ENamedThreads::HighTaskPriority
	);

static TAutoConsoleVariable<float> CVarSecondsBeforeEmbeddedAppSleeps(
	TEXT("tick.SecondsBeforeEmbeddedAppSleeps"),
	1,
	TEXT("When built as embedded, how many ticks to perform before sleeping")
);

/** Task that executes concurrently with Slate when tick.DoAsyncEndOfFrameTasks is true. */
class FExecuteConcurrentWithSlateTickTask
{
	TFunction<void()> TickWithSlate;

public:

	FExecuteConcurrentWithSlateTickTask(TFunction<void()> InTickWithSlate, FEvent* InCompleteEvent)
		: TickWithSlate(InTickWithSlate)
		, CompleteEvent(InCompleteEvent)
	{
		check(CompleteEvent);
	}

	static FORCEINLINE TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FExecuteConcurrentWithSlateTickTask, STATGROUP_TaskGraphTasks);
	}

	static FORCEINLINE ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_AsyncEndOfFrameGameTasks.Get();
	}

	static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		TickWithSlate();
		CompleteEvent->Trigger();
	}

private:
	FEvent* CompleteEvent;
};

// Exits the game/editor if any of the specified phrases appears in the log output
class FOutputDeviceTestExit : public FOutputDevice
{
	TArray<FString> ExitPhrases;
	FString* FoundExitPhrase = nullptr;
public:
	FOutputDeviceTestExit(TArray<FString>&& InExitPhrases)
		: ExitPhrases(InExitPhrases)
	{
	}
	virtual ~FOutputDeviceTestExit()
	{
	}
	bool RequestExit() { return !IsEngineExitRequested() && FoundExitPhrase; }
	FString RequestExitPhrase() { return FoundExitPhrase ? *FoundExitPhrase : FString(); }
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if (FoundExitPhrase || IsEngineExitRequested())
		{
			return;
		}

		for (auto& Phrase : ExitPhrases)
		{
			// look for the exit phrase, but ignore the output of the actual commandline string
			if (FCString::Stristr(V, *Phrase) && !FCString::Stristr(V, TEXT("-testexit=")))
			{
				FoundExitPhrase = &Phrase;
				break;
			}
		}
	}
};

#if WITH_ENGINE && WITH_EDITOR
static FAutoConsoleCommand CmdRunCommandlet(
	TEXT("RunCommandlet"),
	TEXT("<Commandlet> <Args...>. NOTE: This is for debugging/iteration purposes only! Running commandlets multiple times in an editor session may result in issues!"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args) 
		{
			if ( Args.Num() == 0 )
			{
				UE_LOGF(LogInit, Error, "Need to specify a commandlet name to run!");
				return;
			}

			FString CommandletName = Args[0];
			UClass* CommandletClass = nullptr;

			CommandletClass = Cast<UClass>(StaticFindFirstObject(UClass::StaticClass(), *CommandletName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("looking for commandlet")));
			int32 PeriodIdx;
			if (!CommandletClass && CommandletName.FindChar('.', PeriodIdx))
			{
				// try to load module for commandlet specified before a period.
				FModuleManager::Get().LoadModule(*CommandletName.Left(PeriodIdx));
				CommandletClass = FindFirstObject<UClass>(*CommandletName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Looking for commandlet class"));
			}
			if (!CommandletClass)
			{
				UE_LOGF(LogInit, Error, "Commandlet class '%ls' not found!", *CommandletName);
				return;
			}
			TStrongObjectPtr<UCommandlet> Commandlet( NewObject<UCommandlet>(GetTransientPackage(), CommandletClass) );

			// Execute the commandlet.
			double CommandletExecutionStartTime = FPlatformTime::Seconds();
			
			// Commandlets don't always handle -run= properly in the commandline so we'll provide them
			// with a custom version that doesn't have it.
			TArray<FString> CmdlineArgs = Args;
			CmdlineArgs.RemoveAt(0);
			FString CommandletCommandLine = FString::Join(CmdlineArgs, TEXT(" "));
			Commandlet->ParseParms(*CommandletCommandLine);

			PRIVATE_GRunningCommandletClass = CommandletClass;

			{
				SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnCommandletPreMain", CoreDelegatesColor);
				FCoreDelegates::OnCommandletPreMain.Broadcast();
			}

			int32 ErrorLevel;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*WriteToString<512>(TEXT("Commandlet Main "), Commandlet->GetFName()));
				FTrackedActivityScope CommandletActivity(FTrackedActivity::GetEngineActivity(), *FString::Printf(TEXT("Running %s"), *Commandlet->GetName()), false, FTrackedActivity::ELight::Green);
				ErrorLevel = Commandlet->Main(CommandletCommandLine);
			}

			{
				SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnCommandletPostMain", CoreDelegatesColor);
				FCoreDelegates::OnCommandletPostMain.Broadcast();
			}

			PRIVATE_GRunningCommandletClass = nullptr;

			double CommandletExecutionTime = FPlatformTime::Seconds() - CommandletExecutionStartTime;
			UE_LOG(LogInit, Display, LINE_TERMINATOR TEXT("Commandlet returned %d. Execution time:  %.2f seconds"), ErrorLevel, CommandletExecutionTime);
		})
	);
#endif

#if defined(WITH_LAUNCHERCHECK) && defined(WITH_LAUNCHERCHECK_PAKVERIFY) && WITH_LAUNCHERCHECK && WITH_LAUNCHERCHECK_PAKVERIFY
struct FPakValidator
{
	explicit FPakValidator()
	{
		Handle = FCoreDelegates::PakFileMountFailedDelegate.AddLambda([this](const FString& InError) { PakMountErrors.Add(InError); });
	}

	~FPakValidator()
	{
		FCoreDelegates::PakFileMountFailedDelegate.Remove(Handle);
	}

	bool HasPakMountFailed() const { return !PakMountErrors.IsEmpty(); }

	void HandlePakMountError()
	{
		UE_LOGF(LogInit, Warning, "Corrupt or missing files detected: %ls", *FString::Join(PakMountErrors, TEXT(", ")));
		const FText DialogTitle = LOCTEXT("Error.CorruptInstall.Title", "Corrupt/missing files detected");

		if (ILauncherCheckModule::Get().WasRanFromLauncher())
		{
			bool bRunVerifyInstallation = FMessageDialog::Open(
				EAppMsgCategory::Error,
				EAppMsgType::YesNo,
				EAppReturnType::No,
				LOCTEXT("Error.CorruptInstall.VerifyLauncher", "Corrupt or missing files detected. \n\nDo you want to verify your installation?"),
				DialogTitle
			) == EAppReturnType::Yes;
			
			if (bRunVerifyInstallation)
			{ 
				ILauncherCheckModule::Get().RunLauncher(ELauncherAction::AppVerifyInstallation);
			}
			RequestEngineExit(TEXT("Corrupt files detected; running verify in launcher"));
		}
		else
		{
#if UE_BUILD_SHIPPING
			FMessageDialog::Open(
				EAppMsgCategory::Error,
				EAppMsgType::Ok,
				LOCTEXT("Error.CorruptInstall", "Corrupt or missing files detected. Please verify your installation."),
				DialogTitle
			);
			RequestEngineExit(TEXT("Corrupt or missing files detected; exiting"));
#else
			constexpr int MAX_PAK_ERRORS { 10 };
			const FString TruncatedMessage = PakMountErrors.Num() > MAX_PAK_ERRORS ? FString::Format(TEXT("(showing first {0} errors)"), { MAX_PAK_ERRORS }) : TEXT("");
			const TConstArrayView<FString> PakMountErrorsSubset = MakeArrayView(PakMountErrors).Slice(0, FMath::Min(MAX_PAK_ERRORS, PakMountErrors.Num()));
			const FString Message = FString::Format(TEXT("Corrupt or missing files detected {0}:\n{1}\n\nWould you like to continue loading anyway?"),
				{ TruncatedMessage, FString::Join(PakMountErrorsSubset, TEXT("\n")) } );

			bool bRequestExit = FMessageDialog::Open(
				EAppMsgCategory::Error,
				EAppMsgType::YesNo,
				EAppReturnType::No,
				FText::FromString(Message),
				DialogTitle
			) == EAppReturnType::No;

			if (bRequestExit)
			{
				RequestEngineExit(TEXT("Corrupt or missing files detected; exiting"));
			}
#endif
		}
	}

private:
	FDelegateHandle Handle;
	TArray<FString> PakMountErrors;
};

#else
struct FPakValidator 
{
	void HandlePakMountError() {}
	bool HasPakMountFailed() const { return false; }
};

#endif

#if WITH_APPLICATION_CORE
static TUniquePtr<FOutputDeviceConsole>	GScopedLogConsole;
#endif
static TUniquePtr<FOutputDeviceStdOutput> GScopedStdOut;
static TUniquePtr<FOutputDeviceTestExit> GScopedTestExit;

/**
 * Initializes std out device and adds it to GLog
 **/
void InitializeStdOutDevice()
{
	// Check if something is trying to initialize std out device twice.
	check(!GScopedStdOut);

	GScopedStdOut = MakeUnique<FOutputDeviceStdOutput>();
	GLog->AddOutputDevice(GScopedStdOut.Get());
}


bool ParseGameProjectFromCommandLine(const TCHAR* InCmdLine, FString& OutProjectFilePath, FString& OutGameName)
{
	FString ProjectFilePathOption;

	const TCHAR *CmdLine = InCmdLine;
	bool ParseProjectOptionResult = FParse::Value(CmdLine, TEXT("project="), ProjectFilePathOption);
	FString FirstCommandLineToken = FParse::Token(CmdLine, 0);

	// trim any whitespace at edges of string - this can happen if the token was quoted with leading or trailing whitespace
	// VC++ tends to do this in its "external tools" config
	FirstCommandLineToken.TrimStartInline();

	// Output parameters
	OutProjectFilePath = TEXT("");
	OutGameName = TEXT("");

	if ( ( FirstCommandLineToken.Len() && !FirstCommandLineToken.StartsWith(TEXT("-"))) || ParseProjectOptionResult)
	{
		// The first command line argument could be the project file if it exists or the game name if not launching with a project file
		FString ProjectFilePath = ProjectFilePathOption.IsEmpty() ? FString(FirstCommandLineToken) : ProjectFilePathOption;
		if ( FPaths::GetExtension(ProjectFilePath) == FProjectDescriptor::GetExtension() )
		{
			OutProjectFilePath = ProjectFilePath;
			// Here we derive the game name from the project file
			OutGameName = FPaths::GetBaseFilename(OutProjectFilePath);
			return true;
		}
		else if (FPaths::IsRelative(ProjectFilePath) && FPlatformProperties::IsMonolithicBuild() == false && FPlatformProperties::IsMergedModulesBuild() == false)
		{
			// Full game name is assumed to be the first token
			OutGameName = MoveTemp(ProjectFilePath);
			// Derive the project path from the game name. All games must have a uproject file, even if they are in the root folder.
			OutProjectFilePath = FPaths::Combine(*FPaths::RootDir(), *OutGameName, *FString(OutGameName + TEXT(".") + FProjectDescriptor::GetExtension()));
			return true;
		}
	}

#if WITH_EDITOR && WITH_ENGINE
	if (FEditorCommandLineUtils::ParseGameProjectPath(InCmdLine, OutProjectFilePath, OutGameName))
	{
		return true;
	}
#endif
	return false;
}

#if WITH_EDITOR
bool ReadInstalledProjectPath(FString& OutProjFilePath)
{
	if(FApp::IsInstalled())
	{
		FString ProjFilePath;
		if (FFileHelper::LoadFileToString(ProjFilePath, *(FPaths::RootDir() / TEXT("Engine/Build/InstalledProjectBuild.txt"))))
		{
			ProjFilePath.TrimStartAndEndInline();
			if(ProjFilePath.Len() > 0)
			{
				OutProjFilePath = FPaths::RootDir() / ProjFilePath;
				FPaths::NormalizeFilename(OutProjFilePath);
				return true;
			}
		}
	}
	return false;
}
#endif

bool LaunchSetGameName(const TCHAR *InCmdLine, FString& OutGameProjectFilePathUnnormalized)
{
	if (GIsGameAgnosticExe)
	{
		// Initialize GameName to an empty string. Populate it below.
		FApp::SetProjectName(TEXT(""));

		FString ProjFilePath;
		FString LocalGameName;
		if (ParseGameProjectFromCommandLine(InCmdLine, ProjFilePath, LocalGameName) == true)
		{
			// Only set the game name if this is NOT a program...
			if (FPlatformProperties::IsProgram() == false)
			{
				FApp::SetProjectName(*LocalGameName);
			}
			OutGameProjectFilePathUnnormalized = ProjFilePath;
			FPaths::SetProjectFilePath(ProjFilePath);
		}
#if WITH_EDITOR
		else if(ReadInstalledProjectPath(ProjFilePath))
		{
			// Only set the game name if this is NOT a program...
			if (FPlatformProperties::IsProgram() == false)
			{
				FApp::SetProjectName(*FPaths::GetBaseFilename(ProjFilePath));
			}
			OutGameProjectFilePathUnnormalized = ProjFilePath;
			FPaths::SetProjectFilePath(ProjFilePath);
		}
#endif
#if UE_GAME
		else
		{
			// Try to use the executable name as the game name.
			LocalGameName = FPlatformProcess::ExecutableName();
			int32 FirstCharToRemove = INDEX_NONE;
			if (LocalGameName.FindChar(TCHAR('-'), FirstCharToRemove))
			{
				LocalGameName.LeftInline(FirstCharToRemove, EAllowShrinking::No);
			}
			FApp::SetProjectName(*LocalGameName);

			// Check it's not UnrealGame, otherwise assume a uproject file relative to the game project directory
			if (LocalGameName != TEXT("UnrealGame"))
			{
				ProjFilePath = FPaths::Combine(TEXT(".."), TEXT(".."), TEXT(".."), *LocalGameName, *FString(LocalGameName + TEXT(".") + FProjectDescriptor::GetExtension()));
				OutGameProjectFilePathUnnormalized = ProjFilePath;
				FPaths::SetProjectFilePath(ProjFilePath);
			}
		}
#endif

		static bool bPrinted = false;
		if (!bPrinted)
		{
			bPrinted = true;
			if (FApp::HasProjectName())
			{
				UE_LOGF(LogInit, Display, "Running engine for game: %ls", FApp::GetProjectName());
			}
			else
			{
				if (FPlatformProperties::RequiresCookedData())
				{
					UE_LOGF(LogInit, Fatal, "Non-agnostic games on cooked platforms require a uproject file be specified.");
				}
				else
				{
					UE_LOGF(LogInit, Display, "Running engine without a game");
				}
			}
		}
	}
	else
	{
		FString ProjFilePath;
		FString LocalGameName;
		if (ParseGameProjectFromCommandLine(InCmdLine, ProjFilePath, LocalGameName) == true)
		{
			if (FPlatformProperties::RequiresCookedData())
			{
				// Non-agnostic exes that require cooked data cannot load projects, so make sure that the LocalGameName is the GameName
				if (LocalGameName != FApp::GetProjectName())
				{
					UE_LOGF(LogInit, Fatal, "Non-agnostic games cannot load projects on cooked platforms - expected [%ls], found [%ls]", FApp::GetProjectName(), *LocalGameName);
				}
			}
			// Only set the game name if this is NOT a program...
			if (FPlatformProperties::IsProgram() == false)
			{
				FApp::SetProjectName(*LocalGameName);
			}
			OutGameProjectFilePathUnnormalized = ProjFilePath;
			FPaths::SetProjectFilePath(ProjFilePath);
		}

		// In a non-game agnostic exe, the game name should already be assigned by now.
		if (!FApp::HasProjectName())
		{
			UE_LOGF(LogInit, Fatal,"Could not set game name!");
		}
	}

	return true;
}

void LaunchFixProjectPathCase()
{
	if (FPaths::IsProjectFilePathSet())
	{
		FString ProjectFilePath = FPaths::GetProjectFilePath();
		FString ProjectFilePathCorrectCase = FPaths::FindCorrectCase(ProjectFilePath);
		FPaths::SetProjectFilePath(ProjectFilePathCorrectCase);
	}
}

void LaunchFixGameNameCase()
{
#if PLATFORM_DESKTOP && !IS_PROGRAM
	// This is to make sure this function is not misused and is only called when the game name is set
	check(FApp::HasProjectName());

	// correct the case of the game name, if possible (unless we're running a program and the game name is already set)
	if (FPaths::IsProjectFilePathSet())
	{
		const FString GameName(FPaths::GetBaseFilename(FPaths::GetProjectFilePath()));

		const bool bGameNameMatchesProjectCaseSensitive = (FCString::Strcmp(*GameName, FApp::GetProjectName()) == 0);
		if (!bGameNameMatchesProjectCaseSensitive && (FApp::IsProjectNameEmpty() || GIsGameAgnosticExe || (GameName.Len() > 0 && GIsGameAgnosticExe)))
		{
			if (GameName == FApp::GetProjectName()) // case insensitive compare
			{
				FApp::SetProjectName(*GameName);
			}
			else
			{
				const FText Message = FText::Format(
					NSLOCTEXT("Core", "MismatchedGameNames", "The name of the .uproject file ('{0}') must match the name of the project passed in the command line ('{1}')."),
					FText::FromString(*GameName),
					FText::FromString(FApp::GetProjectName()));
				if (!GIsBuildMachine)
				{
					UE_LOGF(LogInit, Warning, "%ls", *Message.ToString());
					FMessageDialog::Open(EAppMsgType::Ok, Message);
				}
				FApp::SetProjectName(TEXT("")); // this disables part of the crash reporter to avoid writing log files to a bogus directory
				if (!GIsBuildMachine)
				{
					exit(1);
				}
				UE_LOGF(LogInit, Fatal, "%ls", *Message.ToString());
			}
		}
	}
#endif	//PLATFORM_DESKTOP
}


static IPlatformFile* ConditionallyCreateFileWrapper(const TCHAR* Name, IPlatformFile* CurrentPlatformFile, const TCHAR* CommandLine, bool* OutFailedToInitialize = nullptr, bool* bOutShouldBeUsed = nullptr )
{
	if (OutFailedToInitialize)
	{
		*OutFailedToInitialize = false;
	}
	if ( bOutShouldBeUsed )
	{
		*bOutShouldBeUsed = false;
	}
	IPlatformFile* WrapperFile = FPlatformFileManager::Get().GetPlatformFile(Name);
	if (WrapperFile != nullptr && WrapperFile->ShouldBeUsed(CurrentPlatformFile, CommandLine))
	{
		if ( bOutShouldBeUsed )
		{
			*bOutShouldBeUsed = true;
		}
		if (WrapperFile->Initialize(CurrentPlatformFile, CommandLine) == false)
		{
			if (OutFailedToInitialize)
			{
				*OutFailedToInitialize = true;
			}
			// Don't delete the platform file. It will be automatically deleted by its module.
			WrapperFile = nullptr;
		}
	}
	else
	{
		// Make sure it won't be used.
		WrapperFile = nullptr;
	}
	return WrapperFile;
}


/**
 * Look for any file overrides on the command line (i.e. network connection file handler)
 */
bool LaunchCheckForFileOverride(const TCHAR* CmdLine, bool& OutFileOverrideFound)
{
	OutFileOverrideFound = false;

	// Get the physical platform file.
	IPlatformFile* CurrentPlatformFile = &FPlatformFileManager::Get().GetPlatformFile();

	// NetworkPlatformFile can be only one of StorageServerClient, StreamingFile or NetworkFile
	// Having a NetworkPlatformFile present prevents creation of Pakfile, CachedReadFile and SandboxFile
	IPlatformFile* NetworkPlatformFile = nullptr;

#if !UE_BUILD_SHIPPING
	if (!NetworkPlatformFile)
	{
		NetworkPlatformFile = ConditionallyCreateFileWrapper(TEXT("StorageServerClient"), CurrentPlatformFile, CmdLine);
		if (NetworkPlatformFile)
		{
			CurrentPlatformFile = NetworkPlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
			// pak streaming currently needs this to be the PlatformPhysical, although it can potentially
			// cause issues for other code that assumes PlatformPhysical only refers to the absolute lowest
			// level for the platform
            if (UE::IsUsingZenPakFileStreaming())
            {
                FPlatformFileManager::Get().SetPlatformPhysical(*CurrentPlatformFile);
            }
		}
	}

	// Streaming network wrapper (it has a priority over normal network wrapper)
	bool bShouldInitializeNetwork = !NetworkPlatformFile;
	while (bShouldInitializeNetwork)
	{
		bool bShouldUseStreamingFile = false;
		NetworkPlatformFile = ConditionallyCreateFileWrapper(TEXT("StreamingFile"), CurrentPlatformFile, CmdLine, &bShouldInitializeNetwork, &bShouldUseStreamingFile);
		if (NetworkPlatformFile)
		{
			CurrentPlatformFile = NetworkPlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
		}

		// if streaming network platform file was tried this loop don't try this one
		// Network file wrapper (only create if the streaming wrapper hasn't been created)
		if (!bShouldUseStreamingFile && !NetworkPlatformFile)
		{
			NetworkPlatformFile = ConditionallyCreateFileWrapper(TEXT("NetworkFile"), CurrentPlatformFile, CmdLine, &bShouldInitializeNetwork);
			if (NetworkPlatformFile)
			{
				CurrentPlatformFile = NetworkPlatformFile;
				FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
			}
		}

		if (bShouldInitializeNetwork)
		{
			FString HostIpString;
			FParse::Value(CmdLine, TEXT("-FileHostIP="), HostIpString);
#if PLATFORM_REQUIRES_FILESERVER
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Failed to connect to file server at %s. RETRYING in 5s.\n"), *HostIpString);
			FPlatformProcess::Sleep(5.0f);
			constexpr uint32 Result = 2;
#else	//PLATFORM_REQUIRES_FILESERVER
			// note that this can't be localized because it happens before we connect to a filserver - localizing would cause ICU to try to load.... from over the file server connection!
			FString Error = FString::Printf(TEXT("Failed to connect to any of the following file servers:\n\n    %s\n\nWould you like to try again? No will fallback to local disk files, Cancel will quit."), *HostIpString.Replace( TEXT("+"), TEXT("\n    ")));
			uint32 Result = FMessageDialog::Open( EAppMsgType::YesNoCancel, FText::FromString( Error ) );
#endif	//PLATFORM_REQUIRES_FILESERVER

			if (Result == EAppReturnType::No)
			{
				break;
			}
			else if (Result == EAppReturnType::Cancel)
			{
				// Cancel - return a failure, and quit
				return false;
			}
		}
	}
#endif
	if (UE::IsUsingZenPakFileStreaming() || !NetworkPlatformFile)
	{
		FPakValidator Validator;
		IPlatformFile* PlatformFile = ConditionallyCreateFileWrapper(TEXT("PakFile"), CurrentPlatformFile, CmdLine);
		if (Validator.HasPakMountFailed())
		{
			Validator.HandlePakMountError();
			if (IsEngineExitRequested())
			{
				return false;
			}
		}
		if (PlatformFile)
		{
			CurrentPlatformFile = PlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
		}
	}

	if (!NetworkPlatformFile)
	{
		IPlatformFile* PlatformFile = ConditionallyCreateFileWrapper(TEXT("CachedReadFile"), CurrentPlatformFile, CmdLine);
		if (PlatformFile)
		{
			CurrentPlatformFile = PlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
		}

		PlatformFile = ConditionallyCreateFileWrapper(TEXT("SandboxFile"), CurrentPlatformFile, CmdLine);
		if (PlatformFile)
		{
			CurrentPlatformFile = PlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
		}
	}

#if !UE_BUILD_SHIPPING
	// Try to create file profiling wrapper
	{
		IPlatformFile* PlatformFile = ConditionallyCreateFileWrapper(TEXT("ProfileFile"), CurrentPlatformFile, CmdLine);
		if (PlatformFile)
		{
			CurrentPlatformFile = PlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
		}
	}
	{
		IPlatformFile* PlatformFile = ConditionallyCreateFileWrapper(TEXT("SimpleProfileFile"), CurrentPlatformFile, CmdLine);
		if (PlatformFile)
		{
			CurrentPlatformFile = PlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
		}
	}
	// Try and create file timings stats wrapper
	{
		IPlatformFile* PlatformFile = ConditionallyCreateFileWrapper(TEXT("FileReadStats"), CurrentPlatformFile, CmdLine);
		if (PlatformFile)
		{
			CurrentPlatformFile = PlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
		}
	}
	// Try and create file open log wrapper (lists the order files are first opened)
	{
		IPlatformFile* PlatformFile = ConditionallyCreateFileWrapper(TEXT("FileOpenLog"), CurrentPlatformFile, CmdLine);
		if (PlatformFile)
		{
			CurrentPlatformFile = PlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
		}
	}
#endif	//#if !UE_BUILD_SHIPPING

	// Wrap the above in a file logging singleton if requested
	{
		IPlatformFile* PlatformFile = ConditionallyCreateFileWrapper(TEXT("LogFile"), CurrentPlatformFile, CmdLine);
		if (PlatformFile)
		{
			CurrentPlatformFile = PlatformFile;
			FPlatformFileManager::Get().SetPlatformFile(*CurrentPlatformFile);
		}
	}

	// If our platform file is different than it was when we started, then an override was used
	OutFileOverrideFound = (CurrentPlatformFile != &FPlatformFileManager::Get().GetPlatformFile());

	// Report if Zen streaming is in use now that configuration is complete.
	CSV_METADATA(TEXT("ZenStreaming"), FPlatformFileManager::Get().FindPlatformFile(TEXT("StorageServer")) == nullptr ? TEXT("0") : TEXT("1"));

	return true;
}

#if !UE_BUILD_SHIPPING
/**
 * Process command line aliases
 *
 */
void LaunchCheckForCommandLineAliases(const FConfigFile& Config, TArray<FString>& PrevExpansions, bool& bChanged)
{
	bChanged = false;

	if (const FConfigSection* Section = Config.FindSection(TEXT("CommandLineAliases")))
	{
		TArray<FString> Tokens;
		{
			const TCHAR* Stream = FCommandLine::Get();
			FString NextToken;
			while (FParse::Token(Stream, NextToken, false))
			{
				Tokens.Add(NextToken);
			}
		}

		for (FConfigSection::TConstIterator ConfigIt(*Section); ConfigIt; ++ConfigIt)
		{
			FString Key = FString("-") + ConfigIt.Key().ToString();
			TArray<FString>::TIterator TokenIt(Tokens);
			while (TokenIt)
			{
				if (PrevExpansions.Contains(*TokenIt))
				{
					TokenIt.RemoveCurrent();
					bChanged = true;
					continue;
				}

				if (*TokenIt == Key)
				{
					PrevExpansions.Add(MoveTemp(*TokenIt));
					*TokenIt = ConfigIt.Value().GetValue();
					bChanged = true;
				}

				++TokenIt;
			}
		}

		if (bChanged)
		{
			FString NewCommandLine = FString::Join(Tokens, TEXT(" "));
			FCommandLine::Set(*NewCommandLine);
		}
	}
}

/**
 * Look for command line file
 *
 */
void LaunchCheckForCmdLineFile(TArray<FString>& PrevExpansions, bool& bChanged)
{
	bChanged = false;

	auto RemoveExpansion = [&bChanged]()
	{
		FString NewCommandLine = FCommandLine::Get();
		FString Left;
		FString Right;
		if (NewCommandLine.Split(TEXT("-CmdLineFile="), &Left, &Right))
		{
			FString NextToken;
			const TCHAR* Stream = *Right;
			if (FParse::Token(Stream, NextToken, /*bUseEscape=*/ false))
			{
				Right = FString(Stream);
			}

			NewCommandLine = Left + TEXT(" ") + Right;
			NewCommandLine.TrimStartAndEndInline();
			FCommandLine::Set(*NewCommandLine);
			bChanged = true;
		}
	};

	auto TryProcessFile = [&RemoveExpansion, &bChanged](const FString& InFilePath)
	{
		FString FileCmds;
		if (FFileHelper::LoadFileToString(FileCmds, *InFilePath))
		{
			UE_LOGF(LogInit, Log, "Inserting commandline from file: %ls, %ls", *InFilePath, *FileCmds);

			FileCmds = FileCmds.TrimStartAndEnd();
			if (FileCmds.Len() == 0)
			{
				RemoveExpansion();
				return true;
			}

			FString NewCommandLine = FCommandLine::Get();
			FString Left;
			FString Right;
			if (NewCommandLine.Split(TEXT("-CmdLineFile="), &Left, &Right))
			{
				FString NextToken;
				const TCHAR* Stream = *Right;
				if (FParse::Token(Stream, NextToken, /*bUseEscape=*/ false))
				{
					Right = FString(Stream);
				}

				NewCommandLine = Left + TEXT(" ") + FileCmds + TEXT(" ") + Right;
				NewCommandLine.TrimStartAndEndInline();
				FCommandLine::Set(*NewCommandLine);
				bChanged = true;
				return true;
			}
		}

		return false;
	};

	FString CmdLineFile;
	while (FParse::Value(FCommandLine::Get(), TEXT("-CmdLineFile="), CmdLineFile))
	{
		if (!CmdLineFile.EndsWith(TEXT(".txt")))
		{
			UE_LOGF(LogInit, Warning, "Can only load commandline files ending with .txt, can't load: %ls", *CmdLineFile);
			RemoveExpansion();
			continue;
		}

		if (PrevExpansions.Contains(CmdLineFile))
		{
			// If already expanded, just remove it
			RemoveExpansion();
			continue;
		}

		bool bFoundFile = TryProcessFile(CmdLineFile);
		if (!bFoundFile && FPaths::ProjectDir().Len() > 0)
		{
			const FString ProjectDir = FPaths::ProjectDir();
			bFoundFile = TryProcessFile(ProjectDir + CmdLineFile);
			if (!bFoundFile)
			{
				const FString ProjectPluginsDir = ProjectDir + TEXT("Plugins/");
				TArray<FString> DirectoryNames;
				IFileManager::Get().IterateDirectory(*ProjectPluginsDir, [&](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
				{
					if (bIsDirectory && TryProcessFile(FString(FilenameOrDirectory) + TEXT("/") + CmdLineFile))
					{
						bFoundFile = true;
						return false;
					}
					return true;
				});
			}
		}

		if (!bFoundFile)
		{
			UE_LOGF(LogInit, Warning, "Failed to load commandline file '%ls'.", *CmdLineFile);
			RemoveExpansion();
			continue;
		}

		PrevExpansions.Add(MoveTemp(CmdLineFile));
	}
}
#endif

bool LaunchHasIncompleteGameName()
{
	if ( FApp::HasProjectName() && !FPaths::IsProjectFilePathSet() )
	{
		// Verify this is a legitimate game name
		// Launched with a game name. See if the <GameName> folder exists. If it doesn't, it could instead be <GameName>Game
		const FString NonSuffixedGameFolder = FPaths::RootDir() / FApp::GetProjectName();
		if (FPlatformFileManager::Get().GetPlatformFile().DirectoryExists(*NonSuffixedGameFolder) == false)
		{
			const FString SuffixedGameFolder = NonSuffixedGameFolder + TEXT("Game");
			if (FPlatformFileManager::Get().GetPlatformFile().DirectoryExists(*SuffixedGameFolder))
			{
				return true;
			}
		}
	}

	return false;
}


void LaunchUpdateMostRecentProjectFile()
{
	// If we are launching without a game name or project file, we should use the last used project file, if it exists
	const FString& AutoLoadProjectFileName = IProjectManager::Get().GetAutoLoadProjectFileName();
	FString RecentProjectFileContents;
	if ( FFileHelper::LoadFileToString(RecentProjectFileContents, *AutoLoadProjectFileName) )
	{
		if ( RecentProjectFileContents.Len() )
		{
			const FString AutoLoadInProgressFilename = AutoLoadProjectFileName + TEXT(".InProgress");
			if ( FPlatformFileManager::Get().GetPlatformFile().FileExists(*AutoLoadInProgressFilename) )
			{
				// We attempted to auto-load a project but the last run did not make it to UEditorEngine::InitEditor.
				// This indicates that there was a problem loading the project.
				// Do not auto-load the project, instead load normally until the next time the editor starts successfully.
				UE_LOGF(LogInit, Display, "There was a problem auto-loading %ls. Auto-load will be disabled until the editor successfully starts up with a project.", *RecentProjectFileContents);
			}
			else if ( FPlatformFileManager::Get().GetPlatformFile().FileExists(*RecentProjectFileContents) )
			{
				// The previously loaded project file was found. Change the game name here and update the project file path
				FApp::SetProjectName(*FPaths::GetBaseFilename(RecentProjectFileContents));
				FPaths::SetProjectFilePath(RecentProjectFileContents);
				UE_LOGF(LogInit, Display, "Loading recent project file: %ls", *RecentProjectFileContents);

				// Write a file indicating that we are trying to auto-load a project.
				// This file prevents auto-loading of projects for as long as it exists. It is a detection system for failed auto-loads.
				// The file is deleted in UEditorEngine::InitEditor, thus if the load does not make it that far then the project will not be loaded again.
				FFileHelper::SaveStringToFile(TEXT(""), *AutoLoadInProgressFilename);
			}
		}
	}
}

#if WITH_ENGINE
void OnStartupContentMounted(FInstallBundleRequestResultInfo Result, bool bDumpEarlyConfigReads, bool bDumpEarlyPakFileReads, bool bReloadConfig, bool bForceQuitAfterEarlyReads);
#endif
void DumpEarlyReads(bool bDumpEarlyConfigReads, bool bDumpEarlyPakFileReads, bool bForceQuitAfterEarlyReads);
void HandleConfigReload(bool bReloadConfig);

#if !UE_BUILD_SHIPPING
class FFileInPakFileHistoryHelper
{
private:
	struct FFileInPakFileHistory
	{
		FString PakFileName;
		FString FileName;
	};
	friend uint32 GetTypeHash(const FFileInPakFileHistory& H)
	{
		uint32 Hash = GetTypeHash(H.PakFileName);
		Hash = HashCombine(Hash, GetTypeHash(H.FileName));
		return Hash;
	}
	friend bool operator==(const FFileInPakFileHistory& A, const FFileInPakFileHistory& B)
	{
		return A.PakFileName == B.PakFileName && A.FileName == B.FileName;
	}

	TSet<FFileInPakFileHistory> History;
	FCriticalSection HistoryLock;

	void OnFileOpenedForRead(const TCHAR* PakFileName, const TCHAR* FileName)
	{
		//UE_LOGF(LogInit, Warning, "OnFileOpenedForRead %u: %ls - %ls", FPlatformTLS::GetCurrentThreadId(), PakFileName, FileName);

		FScopeLock ScopeLock(&HistoryLock);
		History.Emplace(FFileInPakFileHistory{ PakFileName, FileName });
	}

public:
	FFileInPakFileHistoryHelper()
	{
		FCoreDelegates::GetOnFileOpenedForReadFromPakFile().AddRaw(this, &FFileInPakFileHistoryHelper::OnFileOpenedForRead);
	}

	~FFileInPakFileHistoryHelper()
	{
		FCoreDelegates::GetOnFileOpenedForReadFromPakFile().RemoveAll(this);
	}

	void DumpHistory()
	{
		FScopeLock ScopeLock(&HistoryLock);

		History.Sort([](const FFileInPakFileHistory& A, const FFileInPakFileHistory& B)
		{
			if (A.PakFileName == B.PakFileName)
			{
				return A.FileName < B.FileName;
			}

			return A.PakFileName < B.PakFileName;
		});

		const FString SavePath = FPaths::ProjectLogDir() / TEXT("FilesLoadedFromPakFiles.csv");

		FArchive* Writer = IFileManager::Get().CreateFileWriter(*SavePath, FILEWRITE_NoFail);

		auto WriteLine = [Writer](FString&& Line)
		{
			UE_LOGF(LogInit, Display, "%ls", *Line);
			FTCHARToUTF8 UTF8String(*(MoveTemp(Line) + LINE_TERMINATOR));
			Writer->Serialize((UTF8CHAR*)UTF8String.Get(), UTF8String.Length());
		};

		UE_LOGF(LogInit, Display, "Dumping History of files read from Paks to %ls", *SavePath);
		UE_LOGF(LogInit, Display, "Begin History of files read from Paks");
		UE_LOGF(LogInit, Display, "------------------------------------------------------");
		WriteLine(FString::Printf(TEXT("PakFile, File")));
		for (const FFileInPakFileHistory& H : History)
		{
			WriteLine(FString::Printf(TEXT("%s, %s"), *H.PakFileName, *H.FileName));
		}
		UE_LOGF(LogInit, Display, "------------------------------------------------------");
		UE_LOGF(LogInit, Display, "End History of files read from Paks");

		delete Writer;
		Writer = nullptr;
	}
};
TUniquePtr<FFileInPakFileHistoryHelper> FileInPakFileHistoryHelper;
#endif // !UE_BUILD_SHIPPING

void RecordFileReadsFromPaks()
{
#if !UE_BUILD_SHIPPING
	FileInPakFileHistoryHelper = MakeUnique<FFileInPakFileHistoryHelper>();
#endif
}

void DumpRecordedFileReadsFromPaks()
{
#if !UE_BUILD_SHIPPING
	if (FileInPakFileHistoryHelper)
	{
		FileInPakFileHistoryHelper->DumpHistory();
	}
#endif
}

void DeleteRecordedFileReadsFromPaks()
{
#if !UE_BUILD_SHIPPING
	FileInPakFileHistoryHelper = nullptr;
#endif
}



/*-----------------------------------------------------------------------------
	FEngineLoop implementation.
-----------------------------------------------------------------------------*/

FEngineLoop::FEngineLoop()
#if WITH_ENGINE
	: EngineService(nullptr)
#endif
{ }


static FString OriginalProjectModuleName;
static FString ReplacementProjectModuleName;

void FEngineLoop::OverrideProjectModule(const FString& InOriginalProjectModuleName, const FString& InReplacementProjectModuleName)
{
	OriginalProjectModuleName = InOriginalProjectModuleName;
	ReplacementProjectModuleName = InReplacementProjectModuleName;
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("OverrideProjectModule : OriginalProjectModuleName=%s, ReplacementProjectModuleName=%s\n"), *OriginalProjectModuleName, *ReplacementProjectModuleName);

}

int32 FEngineLoop::PreInit(int32 ArgC, TCHAR* ArgV[], const TCHAR* AdditionalCommandline)
{
	FString CmdLine = FCommandLine::BuildFromArgV(nullptr, ArgC, ArgV, AdditionalCommandline);

	// send the command line without the exe name
	return GEngineLoop.PreInit(*CmdLine);
}

#if WITH_ENGINE
bool IsServerDelegateForOSS(FName WorldContextHandle)
{
	if (IsRunningDedicatedServer())
	{
		return true;
	}

	UWorld* World = nullptr;
#if WITH_EDITOR
	if (GEngine && WorldContextHandle != NAME_None)
	{
		const FWorldContext* WorldContext = GEngine->GetWorldContextFromHandle(WorldContextHandle);
		if (WorldContext)
		{
			check(WorldContext->WorldType == EWorldType::Game || WorldContext->WorldType == EWorldType::PIE);
			World = WorldContext->World();
		}		
	}	
#endif

	if (!World)
	{
		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
		if (GameEngine)
		{
			World = GameEngine->GetGameWorld();
		}
		else
		{
			// The calling code didn't pass in a world context and really should have
			if (GIsPlayInEditorWorld)
			{
				World = GWorld;
			}

#if !WITH_DEV_AUTOMATION_TESTS
			// Not having a world to make the right determination is a bad thing
			// In the editor during PIE this will confuse the individual PIE windows and their associated online components
			UE_CLOGF(World == nullptr, LogInit, Error, "Failed to determine if OSS is server in PIE, OSS requests will fail");
#endif
		}
	}

	ENetMode NetMode = World ? World->GetNetMode() : NM_Standalone;
	return (NetMode == NM_ListenServer || NetMode == NM_DedicatedServer);
}
#endif

#if WITH_ENGINE && CSV_PROFILER
static void UpdateCoreCsvStats_BeginFrame()
{
	if (FCsvProfiler::Get()->IsCapturing())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UpdateCoreCsvStats_BeginFrame);
		if (!IsRunningDedicatedServer())
		{
#if !UE_BUILD_SHIPPING
			const uint32 ProcessId = (uint32)FPlatformProcess::GetCurrentProcessId();
		    float ProcessUsagePercent = 0.f, IdleUsagePercent = 0.f;
			if (FPlatformProcess::GetPerFrameProcessorUsage(ProcessId, ProcessUsagePercent, IdleUsagePercent))
			{
				CSV_CUSTOM_STAT_GLOBAL(CPUUsage_Process, ProcessUsagePercent, ECsvCustomStatOp::Set);
				CSV_CUSTOM_STAT_GLOBAL(CPUUsage_Idle, IdleUsagePercent, ECsvCustomStatOp::Set);
			}
#endif
			
		}
#if CSV_PROFILER_ALLOW_DEBUG_FEATURES
		// Handle CsvExecCmds for this frame
		if (GEngine && GWorld)
		{
			TArray<FString> FrameCommands;
			FCsvProfiler::Get()->GetFrameExecCommands(FrameCommands);
			for (FString Cmd : FrameCommands)
			{
				CSV_EVENT_GLOBAL(TEXT("CsvExecCommand : %s"), *Cmd);

				// Try to execute on the local player
				bool bExecuted = false;
				for (FConstPlayerControllerIterator Iterator = GWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
				{
					APlayerController* PlayerController = Iterator->Get();
					if (PlayerController)
					{
						if (ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PlayerController->Player))
						{
							LocalPlayer->Exec(GWorld, *Cmd, *GLog);
							bExecuted = true;
						}
					}
				}
				if (!bExecuted)
				{
					// Fallback to GEngine exec
					GEngine->Exec(GWorld, *Cmd);
				}
			}
		}
#endif // CSV_PROFILER_ALLOW_DEBUG_FEATURES
	}
}

#if !UE_BUILD_SHIPPING
CSV_DEFINE_CATEGORY(GPUUsage, true);
#endif

static void UpdateCoreCsvStats_EndFrame()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateCoreCsvStats_EndFrame);
	if (!IsRunningDedicatedServer())
	{
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(RenderThreadTime, FPlatformTime::ToMilliseconds(GRenderThreadTime), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(GameThreadTime, FPlatformTime::ToMilliseconds(GGameThreadTime), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(GPUTime, FPlatformTime::ToMilliseconds(RHIGetGPUFrameCycles()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(RenderThreadTime_CriticalPath, FPlatformTime::ToMilliseconds(GRenderThreadTimeCriticalPath), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(GameThreadTime_CriticalPath, FPlatformTime::ToMilliseconds(GGameThreadTimeCriticalPath), ECsvCustomStatOp::Set);
		if (IsRunningRHIInSeparateThread())
	    {
			CSV_CUSTOM_STAT_MINIMAL_GLOBAL(RHIThreadTime, FPlatformTime::ToMilliseconds(GRHIThreadTime), ECsvCustomStatOp::Set);
	    }
	    if (GInputLatencyTime > 0)
	    {
			CSV_CUSTOM_STAT_MINIMAL_GLOBAL(InputLatencyTime, FPlatformTime::ToMilliseconds64(GInputLatencyTime), ECsvCustomStatOp::Set);
	    }
	    FPlatformMemoryStats MemoryStats = PlatformMemoryHelpers::GetFrameMemoryStats();
	    float PhysicalMBFree = float(MemoryStats.AvailablePhysical / 1024) / 1024.0f;
#if !UE_BUILD_SHIPPING
	    // Subtract any extra development memory from physical free. This can result in negative values in cases where we would have crashed OOM
	    PhysicalMBFree -= float(FPlatformMemory::GetExtraDevelopmentMemorySize() / 1024ull / 1024ull);
#endif
		CSV_CUSTOM_STAT_MINIMAL_GLOBAL(MemoryFreeMB, PhysicalMBFree, ECsvCustomStatOp::Set);

#if !UE_BUILD_SHIPPING && !CSV_PROFILER_MINIMAL
	    float TargetFPS = 30.0f;
	    static IConsoleVariable* MaxFPSCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("t.MaxFPS"));
	    if (MaxFPSCVar && MaxFPSCVar->GetFloat() > 0)
	    {
		    TargetFPS = MaxFPSCVar->GetFloat();
	    }
	    CSV_CUSTOM_STAT_GLOBAL(MaxFrameTime, 1000.0f / TargetFPS, ECsvCustomStatOp::Set);

		if (GRHISupportsGPUUsage && GNumExplicitGPUsForRendering == 1)
		{
			static const auto CVarPerProcessGPUStats = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.GPUStatisticsPerProcess"));
			FRHIGPUUsageFractions GPUUsage = RHIGetGPUUsage(/* GPUIndex = */ 0);
			CSV_CUSTOM_STAT(GPUUsage, Clock, GPUUsage.ClockScaling * 100.0f, ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(GPUUsage, Usage, GPUUsage.CurrentProcessMHz * 100.0f, ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(GPUUsage, Memory, GPUUsage.CurrentProcessMemoryUsage * 100.0f, ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(GPUUsage, ExternalUsage, GPUUsage.ExternalProcessesMHz * 100.0f, ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(GPUUsage, ExternalMemory, GPUUsage.ExternalProcessMemoryUsage * 100.0f, ECsvCustomStatOp::Set);
		}
#endif
	}
}
#endif // WITH_ENGINE && CSV_PROFILER

#if WITH_ENGINE
namespace AppLifetimeEventCapture
{
	static void AppWillDeactivate()
	{
		UE_LOGF( LogCore, Display, "AppLifetime: Application will deactivate" );
		CSV_EVENT_GLOBAL(TEXT("App_WillDeactivate"));
	}

	static void AppHasReactivated()
	{
		UE_LOGF( LogCore, Display, "AppLifetime: Application has reactivated" );
		CSV_EVENT_GLOBAL(TEXT("App_HasReactivated"));
	}

	static void AppWillEnterBackground()
	{
		UE_LOGF( LogCore, Display, "AppLifetime: Application will enter background" );
		CSV_EVENT_GLOBAL(TEXT("App_WillEnterBackground"));
	}

	static void AppHasEnteredForeground()
	{
		UE_LOGF( LogCore, Display, "AppLifetime: Application has entered foreground" );
		CSV_EVENT_GLOBAL(TEXT("App_HasEnteredForeground"));
	}

	static void Init()
	{
		FCoreDelegates::ApplicationWillDeactivateDelegate.AddStatic(AppWillDeactivate);
		FCoreDelegates::ApplicationHasReactivatedDelegate.AddStatic(AppHasReactivated);
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddStatic(AppWillEnterBackground);
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddStatic(AppHasEnteredForeground);
	}
}
#endif //WITH_ENGINE

static void UpdateGInputTime()
{
	GInputTime = FPlatformTime::Cycles64();
}

static TArray<FString> TokenizeCommandline(const TCHAR* CmdLine, bool bRetainQuotes)
{
	TArray<FString> TokenArray;

	const TCHAR* ParsedCmdLine = CmdLine;

	while (*ParsedCmdLine)
	{
		FString Token;
				
		// skip over whitespace to look for a quote
		while (FChar::IsWhitespace(*ParsedCmdLine))
		{
			ParsedCmdLine++;
		}

		// if we want to keep quotes around the token, and the first character is a quote, then FToken::Parse
		// will remove the quotes, so put them back
		if (bRetainQuotes && (*ParsedCmdLine == TEXT('"')))
		{
			FParse::Token(ParsedCmdLine, Token, 0);
			Token = FString::Printf(TEXT("\"%s\""), *Token);
		}
		else
		{
			FParse::Token(ParsedCmdLine, Token, 0);
			Token.TrimStartAndEndInline();
		}

		if (!Token.IsEmpty())
		{
			TokenArray.Add(MoveTemp(Token));
		}
	}

	return MoveTemp(TokenArray);
}

/** Enumeration representing the type of the command-line argument representing the game (typically the first argument). */
enum class EGameStringType
{
	GameName,
	ProjectPath,
	ProjectShortName,
	Unknown
};

/**
  * Finds a command-line argument representing the game, removes it from the array and returns.
  *
 * @param TokenArray    Array of the tokenized command line
 * @param QuotedTokenArray  Parellel array that maintains quotes around params
  * @param OutStringType Type of the game string found.
  * @return String representing the game command-line parameter; empty string if not found (OutStringType == Unknown in such case).
  */
static FString ExtractGameStringArgument(TArray<FString>& TokenArray, TArray<FString>& QuotedTokenArray, EGameStringType& OutStringType)
{
	for (int32 I = 0; I < TokenArray.Num(); ++I)
	{
		FString NormalizedToken = TokenArray[I];

		// Path returned by FPaths::GetProjectFilePath() is normalized, so may have symlinks and ~ resolved and may differ from the original path to .uproject passed in the command line
		FPaths::NormalizeFilename(NormalizedToken);

		const bool bTokenIsGameName                 = (FApp::HasProjectName()         && TokenArray[I]   == FApp::GetProjectName());
		const bool bTokenIsGameProjectFilePath      = (FPaths::IsProjectFilePathSet() && NormalizedToken == FPaths::GetProjectFilePath());
		const bool bTokenIsGameProjectFileShortName = (FPaths::IsProjectFilePathSet() && TokenArray[I]   == FPaths::GetCleanFilename(FPaths::GetProjectFilePath()));

		if (bTokenIsGameName || bTokenIsGameProjectFilePath || bTokenIsGameProjectFileShortName)
		{
			if (bTokenIsGameName)
			{
				OutStringType = EGameStringType::GameName;
			}
			else if (bTokenIsGameProjectFilePath)
			{
				OutStringType = EGameStringType::ProjectPath;
			}
			else if (bTokenIsGameProjectFileShortName)
			{
				OutStringType = EGameStringType::ProjectShortName;
			}

			FString Result = TokenArray[I];

			TokenArray.RemoveAt(I);
			QuotedTokenArray.RemoveAt(I);

			return Result;
		}
	}

	OutStringType = EGameStringType::Unknown;

	return FString();
}

#if WITH_EDITOR && WITH_ENGINE
static void GenerateShaderHeaders()
{
	//Generate shader headers for all the shader platforms. It's necessary for the cook. I don't do it in the cook commandlet
	//because it's not early enough.
	ITargetPlatformManagerModule* PlatformModule = GetTargetPlatformManager();
	if (PlatformModule)
	{
		const TArray<ITargetPlatform*>& ActivePlatforms = PlatformModule->GetActiveTargetPlatforms();
		for (const ITargetPlatform* Platform : ActivePlatforms)
		{
			TArray<FName> ShaderPlatformNameArray;
			Platform->GetAllTargetedShaderFormats(ShaderPlatformNameArray);

			for (const FName& ShaderPlatformName : ShaderPlatformNameArray)
			{
				FShaderCompileUtilities::GenerateBrdfHeaders(ShaderPlatformName);
			}
		}
	}
}
#endif // #if WITH_EDITOR

DECLARE_CYCLE_STAT(TEXT("FEngineLoop::PreInitPreStartupScreen.AfterStats"), STAT_FEngineLoop_PreInitPreStartupScreen_AfterStats, STATGROUP_LoadTime);
DECLARE_CYCLE_STAT(TEXT("FEngineLoop::PreInitPostStartupScreen.AfterStats"), STAT_FEngineLoop_PreInitPostStartupScreen_AfterStats, STATGROUP_LoadTime);

int32 FEngineLoop::PreInitPreStartupScreen(const TCHAR* CmdLine)
{
	ON_SCOPE_EXIT { FBootProfiling::OnPreInitPreStartupScreenComplete(); };
	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::StartOfEnginePreInit);
#if WITH_ENGINE && CSV_PROFILER
	// PreInit the CSV profiler as early as possible in case we're boot profiling with -csvCaptureFrames
	FCsvProfiler::PreInit(CmdLine);
#endif
	SCOPED_BOOT_TIMING("FEngineLoop::PreInitPreStartupScreen");

	// GLog is initialized lazily and its default primary thread is the thread that initialized it.
	// This lazy initialization can happen during initialization of a DLL, which Windows does on a
	// worker thread, which makes that worker thread the primary thread. Make this the primary thread
	// until initialization is far enough along to try to start a dedicated primary thread.
	GLog->SetCurrentThreadAsPrimaryThread();

#if IS_PROGRAM
	// Non-program targets have the backlog enabled by default. Enable it as early as possible here for
	// cases where it is disabled by default. It will be disabled when module loading is complete.
	// Do not disable the backlog if the program enabled it explicitly before calling PreInit().
	const bool bDisableBacklog = !GLog->EnableBacklog(true);
	ON_SCOPE_EXIT
	{
		if (bDisableBacklog)
		{
			GLog->EnableBacklog(false);
		}
	};
#endif

	// Command line option for enabling named events
	if (FParse::Param(CmdLine, TEXT("statnamedevents")))
	{
		++GCycleStatsShouldEmitNamedEvents;
	}
	
	if (FParse::Param(CmdLine, TEXT("verbosenamedevents")))
	{
		++GCycleStatsShouldEmitNamedEvents;
		GShouldEmitVerboseNamedEvents = true;
	}


	// Set the flag for whether we've build DebugGame instead of Development. The engine does not know this (whereas the launch module does) because it is always built in development.
#if UE_BUILD_DEVELOPMENT && defined(UE_BUILD_DEVELOPMENT_WITH_DEBUGGAME) && UE_BUILD_DEVELOPMENT_WITH_DEBUGGAME
	FApp::SetDebugGame(true);
#endif

#if PLATFORM_WINDOWS
	// Register a handler for Ctrl-C so we've effective signal handling from the outset.
	FWindowsPlatformMisc::SetGracefulTerminationHandler();
#endif // PLATFORM_WINDOWS

#if BUILD_EMBEDDED_APP
#ifdef EMBEDDED_LINKER_GAME_HELPER_FUNCTION
	extern void EMBEDDED_LINKER_GAME_HELPER_FUNCTION();
	EMBEDDED_LINKER_GAME_HELPER_FUNCTION();
#endif
	FEmbeddedCommunication::Init();
	FEmbeddedCommunication::KeepAwake(TEXT("Startup"), false);
#endif

	FMemory::SetupTLSCachesOnCurrentThread();

	FPlatformMisc::SetUTF8Output();

#if !UE_BUILD_SHIPPING
	if (FParse::Param(CmdLine, TEXT("IgnoreDebugger")))
	{
		GIgnoreDebugger = true;
	}
#endif // !UE_BUILD_SHIPPING

	// Switch into executable's directory.
#if !defined(DISABLE_CWD_CHANGES) || DISABLE_CWD_CHANGES==0
	FPlatformProcess::SetCurrentWorkingDirectoryToBaseDir();
#endif

	// this is set later with shorter command lines, but we want to make sure it is set ASAP as some subsystems will do the tests themselves...
	// also realize that command lines can be pulled from the network at a slightly later time.
	if (!FCommandLine::Set(CmdLine))
	{
		// Fail, shipping builds will crash if setting command line fails
		return -1;
	}

	// Avoiding potential exploits by not exposing command line overrides in the shipping games.
#if !UE_BUILD_SHIPPING && WITH_EDITORONLY_DATA
	// Retrieve additional command line arguments from environment variable.
	FString Env = FPlatformMisc::GetEnvironmentVariable(TEXT("UE-CmdLineArgs")).TrimStart();
	if (Env.Len())
	{
		UE_LOGF(LogInit, Log, "Inserting commandline from UE-CmdLineArgs: %ls", *Env);

		// Append the command line environment after inserting a space as we can't set it in the
		// environment.
		FCommandLine::Append(TEXT(" -EnvAfterHere "));
		FCommandLine::Append(*Env);
		CmdLine = FCommandLine::Get();
	}
#endif

	// Name of project file before normalization (as specified in command line).
	// Used to fixup project name if necessary.
	FString GameProjectFilePathUnnormalized;

	{
		SCOPED_BOOT_TIMING("LaunchSetGameName");

		// Set GameName, based on the command line
		if (LaunchSetGameName(CmdLine, GameProjectFilePathUnnormalized) == false)
		{
			// If it failed, do not continue
			return 1;
		}
	}

	// Initialize trace
	FTraceAuxiliary::Initialize(CmdLine);
	FTraceAuxiliary::TryAutoConnect();

	// disable/enable LLM based on commandline
	{
		SCOPED_BOOT_TIMING("LLM Init");
		LLM(FLowLevelMemTracker::Get().ProcessCommandLine(CmdLine));
#if MEMPRO_ENABLED
		FMemProProfiler::Init(CmdLine);
#endif
	}
	LLM_SCOPE(ELLMTag::EnginePreInitMemory);

	{
		SCOPED_BOOT_TIMING("InitTaggedStorage");
		FPlatformMisc::InitTaggedStorage(1024);
	}

#if WITH_ENGINE
	FCoreUObjectDelegates::PostGarbageCollectConditionalBeginDestroy.AddStatic(DeferredPhysResourceCleanup);
#endif
	FCoreDelegates::OnSamplingInput.AddStatic(UpdateGInputTime);

#if defined(WITH_LAUNCHERCHECK) && WITH_LAUNCHERCHECK && !WITH_EDITOR
	if (ILauncherCheckModule::Get().WasRanFromLauncher() == false)
	{
		// Tell Launcher to run us instead
		ILauncherCheckModule::Get().RunLauncher(ELauncherAction::AppLaunch);
		// We wish to exit
		RequestEngineExit(TEXT("Run outside of launcher; restarting via launcher"));
		return 1;
	}
#endif

#if WITH_APPLICATION_CORE
	{
		SCOPED_BOOT_TIMING("CreateConsoleOutputDevice");
		// Initialize log console here to avoid statics initialization issues when launched from the command line.
		GScopedLogConsole = TUniquePtr<FOutputDeviceConsole>(FPlatformApplicationMisc::CreateConsoleOutputDevice());
	}
#endif

	// Initialize std out device as early as possible if requested in the command line
#if PLATFORM_DESKTOP
	// consoles don't typically have stdout, and FOutputDeviceDebug is responsible for echoing logs to the
	// terminal
	if (FParse::Param(FCommandLine::Get(), TEXT("stdout")))
	{
		InitializeStdOutDevice();
	}
#endif

	if (FParse::Param(CmdLine, TEXT("UTF8Output")))
	{
		// UE_DEPRECATED(5.6, "This comment exists to describe when -UTF8Output was deprecated and allow finding it in a search for UE 5.6 deprecations.")
		UE_LOGF(LogInit, Display, "Output is UTF-8 by default. Use of -UTF8Output on the command is deprecated and can be removed.");
	}

// If we are in Debug, Development, or Test/Shipping with ALLOW_PROFILEGPU... enabled, then automatically allow draw events.
// Command lines allow disabling of draw events if WITH_PROGILEGPU is enabled. Test/Shipping on their own require explicit opt in.
#if WITH_PROFILEGPU	
	if (!FParse::Param(FCommandLine::Get(), TEXT("nodrawevents")))
	{
		SetEmitDrawEvents(true);
	}
// Continue to protect shipping build from emitdrawevents (if needed in shipping, ALLOW_PROFILEGPU_IN_SHIPPING should be used instead to enable WITH_PROFILEGPU)
#elif !UE_BUILD_SHIPPING
	if (FParse::Param(FCommandLine::Get(), TEXT("emitdrawevents")))
	{
		SetEmitDrawEvents(true);
	}
#endif

#if !UE_BUILD_SHIPPING
	if (FPlatformProperties::SupportsQuit())
	{
		FString ExitPhrases;
		if (FParse::Value(FCommandLine::Get(), TEXT("testexit="), ExitPhrases))
		{
			TArray<FString> ExitPhrasesList;
			if (ExitPhrases.ParseIntoArray(ExitPhrasesList, TEXT("+"), true) > 0)
			{
				GScopedTestExit = MakeUnique<FOutputDeviceTestExit>(MoveTemp(ExitPhrasesList));
				GLog->AddOutputDevice(GScopedTestExit.Get());
			}
		}
	}

	// Activates malloc frame profiler from the command line 
	// Recommend enabling bGenerateSymbols to ensure callstacks can resolve and bRetainFramePointers to ensure frame pointers remain valid.
	// Also disabling the hitch detector ALLOW_HITCH_DETECTION=0 helps ensure quicker more accurate runs.
	if (FParse::Param(FCommandLine::Get(), TEXT("mallocframeprofiler")))
	{
		GMallocFrameProfilerEnabled = true;
		UE::Private::GMalloc = FMallocFrameProfiler::OverrideIfEnabled(UE::Private::GMalloc);
	}
#endif // !UE_BUILD_SHIPPING

	// Switch into executable's directory (may be required by some of the platform file overrides)
#if !defined(DISABLE_CWD_CHANGES) || DISABLE_CWD_CHANGES==0
	FPlatformProcess::SetCurrentWorkingDirectoryToBaseDir();
#endif

	// This fixes up the relative project path, needs to happen before we set platform file paths
	if (FPlatformProperties::IsProgram() == false)
	{
		SCOPED_BOOT_TIMING("Fix up the relative project path");

		if (FPaths::IsProjectFilePathSet())
		{
			FString ProjPath = FPaths::GetProjectFilePath();
			if (FPaths::FileExists(ProjPath) == false)
			{
				// display it multiple ways, it's very important error message...
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Project file not found: %s\n"), *ProjPath);
				UE_LOGF(LogInit, Display, "Project file not found: %ls", *ProjPath);
				UE_LOGF(LogInit, Display, "\tAttempting to find via project info helper.");
				// Use the uprojectdirs
				FString GameProjectFile = FUProjectDictionary::GetDefault().GetRelativeProjectPathForGame(FApp::GetProjectName(), FPlatformProcess::BaseDir());
				if (GameProjectFile.IsEmpty() == false)
				{
					UE_LOGF(LogInit, Display, "\tFound project file %ls.", *GameProjectFile);
					FPaths::SetProjectFilePath(GameProjectFile);

					// Fixup command line if project file wasn't found in specified directory to properly parse next arguments.
					FString OldCommandLine = FString(FCommandLine::Get());
					OldCommandLine.ReplaceInline(*GameProjectFilePathUnnormalized, *GameProjectFile, ESearchCase::CaseSensitive);
					FCommandLine::Set(*OldCommandLine);
					CmdLine = FCommandLine::Get();
				}
			}
		}
	}

#if !IS_PROGRAM && !IS_MONOLITHIC
	FString EngineBinariesRootDirectory = FPlatformMisc::EngineDir();
#endif
	FString ProjectBinariesRootDirectory;
	if (FApp::HasProjectName())
	{
		ProjectBinariesRootDirectory = FPlatformMisc::ProjectDir();
#if !IS_MONOLITHIC
#if !IS_PROGRAM
		FPlatformMisc::GetEngineAndProjectAbsoluteDirsFromExecutable(ProjectBinariesRootDirectory, EngineBinariesRootDirectory);
#endif
		
		// Strip suffix "-Platform-Config" to allow building module name to load: "App" + "-ModuleName" + "-Platform-Config"
		const FString ExecutableNameWithoutExtension = FPlatformProcess::ExecutableName();
		const FStringView BaseExecutableName = FPathViews::GetBaseFilename(ExecutableNameWithoutExtension);
		FStringView BasePrefix = BaseExecutableName;
		FString BaseSuffix;
		if (FApp::GetBuildConfiguration() != EBuildConfiguration::Development)
		{
			int32 LastDash = INDEX_NONE;
			if (BasePrefix.FindLastChar(TEXT('-'), LastDash))
			{
				BasePrefix.LeftInline(LastDash);

				LastDash = INDEX_NONE;
				if (BasePrefix.FindLastChar(TEXT('-'), LastDash))
				{
					BasePrefix.LeftInline(LastDash);
				}
			}

			BaseSuffix = BaseExecutableName.RightChop(BasePrefix.Len());
		}

		const FString PathPrefix = FPaths::Combine(FPathViews::GetPath(ExecutableNameWithoutExtension), BasePrefix);

		// Loading preinit/common module if it exists.
		// PreInit module can contain things like decryption logic for pak files and things that is project specific but needs to initialize very early
		// Common module is a merged binary containing all the important modules for the project
		TArray<FString> FileNames =
		{
#if UE_MERGED_MODULES
			FString::Printf(TEXT("%s-Common%s.%s"), *PathPrefix, *BaseSuffix, FPlatformProcess::GetModuleExtension()),
			FString::Printf(TEXT("%s-Common-%s-%s.%s"), FPlatformProcess::ExecutableName(), FPlatformProcess::GetBinariesSubdirectory(), LexToString(FApp::GetBuildConfiguration()), FPlatformProcess::GetModuleExtension()),
#else
			FString::Printf(TEXT("%s-%sPreInit%s.%s"), *PathPrefix, FApp::GetProjectName(), *BaseSuffix, FPlatformProcess::GetModuleExtension()),
			FString::Printf(TEXT("%s-%sPreInit-%s-%s.%s"), *FApp::GetName(), FApp::GetProjectName(), FPlatformProcess::GetBinariesSubdirectory(), LexToString(FApp::GetBuildConfiguration()), FPlatformProcess::GetModuleExtension())
#endif
		};
		
		// UE-230629: On mac UnrealLightmass this would cause assert on main thread
#if PLATFORM_MAC && WITH_APPLICATION_CORE
		if (!FMacPlatformApplicationMisc::IsMainThread())
		{
#endif
		FTaskTagScope scope(ETaskTag::EStaticInit);
#if PLATFORM_MAC && WITH_APPLICATION_CORE
		}
#endif

		for (const FString& FileName : FileNames)
		{
			FString ModulePath = FPaths::Combine(ProjectBinariesRootDirectory, "Binaries", FPlatformProcess::GetBinariesSubdirectory(), FileName);
			if (FPaths::FileExists(ModulePath))
			{
				FPlatformProcess::GetDllHandle(*ModulePath);
				break;
			}
		}
#endif
	}

	// Output devices.
	{
		SCOPED_BOOT_TIMING("Init Output Devices");
#if WITH_APPLICATION_CORE
		GError = FPlatformApplicationMisc::GetErrorOutputDevice();
		GWarn = FPlatformApplicationMisc::GetFeedbackContext();
#else
		GError = FPlatformOutputDevices::GetError();
		GWarn = FPlatformOutputDevices::GetFeedbackContext();
#endif

		{
			SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnOutputDevicesInit", CoreDelegatesColor);
			FCoreDelegates::OnOutputDevicesInit.Broadcast();
		}
	}

	// Avoiding potential exploits by not exposing command line overrides in the shipping games.
#if !UE_BUILD_SHIPPING
	{
		SCOPED_BOOT_TIMING("Command Line Adjustments");

		FConfigFile CommandLineAliasesConfigFile;
		if (FPaths::ProjectDir().Len() > 0)
		{
			// Pass EngineIntermediateDir as GeneratedConfigDir of FPaths::GeneratedConfigDir() so that saved directory is not cached before -saveddir argument can be added
			FConfigCacheIni::LoadExternalIniFile(CommandLineAliasesConfigFile, TEXT("CommandLineAliases"), nullptr, *FPaths::Combine(FPaths::ProjectDir(), TEXT("Config")),
												 /*bIsBaseIniName=*/ false,
												 /*Platform=*/ nullptr,
												 /*bForceReload=*/ false,
												 /*bWriteDestIni=*/ false,
												 /*bAllowGeneratedIniWhenCooked=*/ true,
												 /*GeneratedConfigDir=*/ *FPaths::EngineIntermediateDir());
		}

		TArray<FString> PrevAliasExpansions;
		TArray<FString> PrevCmdLineFileExpansions;

		for (;;)
		{
			bool bExpandedAliases = false;
			LaunchCheckForCommandLineAliases(CommandLineAliasesConfigFile, PrevAliasExpansions, bExpandedAliases);

			bool bExpandedCmdLineFile = false;
			LaunchCheckForCmdLineFile(PrevCmdLineFileExpansions, bExpandedCmdLineFile);

			if (!bExpandedAliases && !bExpandedCmdLineFile)
			{
				break;
			}
		}

		// Refresh local CmdLine: the alias/file-expansion loop above and OnOutputDevicesInit
		// may have called FCommandLine::Set/Append internally.
		CmdLine = FCommandLine::Get();
	}
#endif

#if USE_IO_DISPATCHER
	if (FIoStatus Status = FIoDispatcher::Initialize(); !Status.IsOk())
	{
		UE_LOGF(LogInit, Error, "Failed to initialize I/O dispatcher: '%ls'", *Status.ToString());
		return 1;
	}
#endif

	{
		SCOPED_BOOT_TIMING("BeginPreInitTextLocalization");
		BeginPreInitTextLocalization();
	}

#if WITH_ENGINE
	{
		SCOPED_BOOT_TIMING("PreInitShaderLibrary");
		FShaderCodeLibrary::PreInit();
	}
#endif // WITH_ENGINE

	// allow the command line to override the platform file singleton
	bool bFileOverrideFound = false;
	bool bErrorExitAfterSettingUpLogging = false;
	{
		SCOPED_BOOT_TIMING("LaunchCheckForFileOverride");
		if (LaunchCheckForFileOverride(CmdLine, bFileOverrideFound) == false)
		{
			// We will crash at some point. Try to reach log initialization to record the buffered logs and then exit.
			bErrorExitAfterSettingUpLogging = true;
		}
	}

	// 
#if PLATFORM_DESKTOP && !IS_MONOLITHIC
	{
		SCOPED_BOOT_TIMING("AddExtraBinarySearchPaths");
		FModuleManager::Get().AddExtraBinarySearchPaths();
	}
#endif

	// Initialize file manager
	{
		SCOPED_BOOT_TIMING("IFileManager::Get().ProcessCommandLineOptions");
		IFileManager::Get().ProcessCommandLineOptions();
	}

#if WITH_COREUOBJECT
	{
		SCOPED_BOOT_TIMING("InitializeNewAsyncIO");
		FPlatformFileManager::Get().InitializeNewAsyncIO();
	}
#endif

	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::FileSystemReady);

	if (GIsGameAgnosticExe)
	{
		// If we launched without a project file, but with a game name that is incomplete, warn about the improper use of a Game suffix
		if (LaunchHasIncompleteGameName())
		{
			// We did not find a non-suffixed folder and we DID find the suffixed one.
			// The engine MUST be launched with <GameName>Game.
			const FText GameNameText = FText::FromString(FApp::GetProjectName());
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("RequiresGamePrefix", "Error: UnrealEditor does not append 'Game' to the passed in game name.\nYou must use the full name.\nYou specified '{0}', use '{0}Game'."), GameNameText));
			// We will crash at some point. Try to reach log initialization to record the buffered logs and then exit.
			bErrorExitAfterSettingUpLogging = true;
		}
	}

	// remember thread id of the main thread
	GGameThreadId = FPlatformTLS::GetCurrentThreadId();
	GIsGameThreadIdInitialized = true;

	FPlatformProcess::SetThreadPriority(FPlatformAffinity::GetGameThreadPriority());
	FPlatformProcess::SetThreadAffinityMask(FPlatformAffinity::GetMainGameMask());
	FPlatformProcess::SetupGameThread();

	// These bools are the mode selector and are mutually exclusive. This function selects the mode by calling one of the 
	// SetIsRunningAs* lambdas, which it does based on commandline and configuration considerations in a specific order.
	// TODO: Convert these bools to an enum since they are mutually exclusive.
	bool bHasCommandletToken = false;
	bool bHasEditorToken = false;
	bool bIsRunningAsDedicatedServer = false;
	bool bIsRegularClient = false;

	FString TokenToForward;

	// these tokens are use later to restore a single commandline string, so keep a version that has quotes around tokens,
	// in case there are spaces in a token which will break parsing that single string
	TArray<FString> TokenArray     		= TokenizeCommandline(CmdLine, false);
	TArray<FString> QuotedTokenArray    = TokenizeCommandline(CmdLine, true);

	EGameStringType GameStringType = EGameStringType::Unknown;
	FString         GameString     = ExtractGameStringArgument(TokenArray, QuotedTokenArray, GameStringType);

	auto IsModeSelected = [&bHasCommandletToken, &bHasEditorToken, &bIsRegularClient, &bIsRunningAsDedicatedServer]()
	{
		return bHasCommandletToken || bHasEditorToken || bIsRegularClient || bIsRunningAsDedicatedServer;
	};
#if UE_EDITOR || WITH_ENGINE
	FString CommandletCommandLine;
	auto SetIsRunningAsCommandlet = [&CommandletCommandLine, &QuotedTokenArray,
		&bHasCommandletToken, &bIsRunningAsDedicatedServer, &bHasEditorToken, &bIsRegularClient, &IsModeSelected, &TokenToForward]
		(FStringView CommandletName)
	{
		checkf(!IsModeSelected(), TEXT("SetIsRunningAsCommandlet should not be called after mode has been selected."));
		bHasCommandletToken = true;

		TokenToForward = CommandletName;

		GIsClient = true;
		GIsServer = true;
#if WITH_EDITORONLY_DATA
		GIsEditor = true;
#endif
#if STATS
		// Leave the stats enabled.
		if (!UE::Stats::FStats::EnabledForCommandlet())
		{
			FThreadStats::PrimaryDisableForever();
		}
#endif

		CommandletCommandLine = FString::Join(QuotedTokenArray, TEXT(" "));
#if WITH_EDITOR && WITH_ENGINE
		if (CommandletName == TEXTVIEW("cookcommandlet") || FParse::Param(FCommandLine::Get(), TEXT("RunAsCookCommandlet")))
		{
			PRIVATE_GIsRunningCookCommandlet = true;
			UE::Cook::InitializeCookGlobals();
			PRIVATE_GIsRunningDLCCookCommandlet = (FCString::Strifind(FCommandLine::Get(), TEXT("-dlcname=")) != nullptr);
		}
#endif
#if WITH_ENGINE
		PRIVATE_GIsRunningCommandlet = true;
#endif
		// Allow commandlet rendering and/or audio based on command line switch (too early to let the commandlet itself override this).
		PRIVATE_GAllowCommandletRendering = FParse::Param(FCommandLine::Get(), TEXT("AllowCommandletRendering"));
		PRIVATE_GAllowCommandletAudio = FParse::Param(FCommandLine::Get(), TEXT("AllowCommandletAudio"));
	};
#endif // UE_EDITOR || WITH_ENGINE || WITH_EDITOR
	auto SetIsRunningAsRegularClient = [&IsModeSelected, &bIsRegularClient, &TokenArray, &TokenToForward]()
	{
		checkf(!IsModeSelected(), TEXT("SetIsRunningAsRegularClient should not be called after mode has been selected."));
		bIsRegularClient = true;

		// Take the first non-switch command-line parameter (i.e. one not starting with a dash) and remember
		// it to check later as it could be a mistyped commandlet (short name).
		const FString* NonSwitchParam = TokenArray.FindByPredicate(
			[](const FString& Token)
			{
				return Token[0] != TCHAR('-');
			}
		);

		if (NonSwitchParam)
		{
			TokenToForward = *NonSwitchParam;
		}

		GIsClient = true;
		GIsServer = false;
#if WITH_EDITORONLY_DATA
		GIsEditor = false;
#endif
#if WITH_ENGINE
		checkf(!PRIVATE_GIsRunningCommandlet, TEXT("It should not be possible for PRIVATE_GIsRunningCommandlet to have been set when calling SetIsRunningAsRegularClient"));
#endif
	};
	auto SetIsRunningAsDedicatedServer = [&IsModeSelected, &bIsRunningAsDedicatedServer]()
	{
		checkf(!IsModeSelected(), TEXT("SetIsRunningAsDedicatedServer should not be called after mode has been selected."));
		bIsRunningAsDedicatedServer = true;

		GIsClient = false;
		GIsServer = true;
#if WITH_EDITORONLY_DATA
		GIsEditor = false;
#endif
#if WITH_ENGINE
		checkf(!PRIVATE_GIsRunningCommandlet, TEXT("It should not be possible for PRIVATE_GIsRunningCommandlet to have been set when calling SetIsRunningAsDedicatedServer"));
#endif
	};
#if WITH_EDITORONLY_DATA
	auto SetIsRunningAsEditor = [&IsModeSelected, &bHasEditorToken]()
	{
		checkf(!IsModeSelected(), TEXT("SetIsRunningAsEditor should not be called after mode has been selected."));
		bHasEditorToken = true;

		GIsClient = true;
		GIsServer = true;
		GIsEditor = true;
#if WITH_ENGINE
		checkf(!PRIVATE_GIsRunningCommandlet, TEXT("It should not be possible for PRIVATE_GIsRunningCommandlet to have been set when calling SetIsRunningAsEditor"));
#endif
	};
#endif

	if (IsRunningDedicatedServer())
	{
		SetIsRunningAsDedicatedServer();
	}

#if UE_EDITOR || WITH_ENGINE
	TArray<FString> NonSwitchTokens;
	TArray<FString> Switches;
	UCommandlet::ParseCommandLine(CmdLine, NonSwitchTokens, Switches);
	if (!IsModeSelected())
	{
		for (const FString& ParsedToken : NonSwitchTokens)
		{
			if (ParsedToken.EndsWith(TEXT("Commandlet")))
			{
				SetIsRunningAsCommandlet(ParsedToken.TrimStartAndEnd());
				break;
			}
		}
		if (!bHasCommandletToken)
		{
			for (const FString& ParsedSwitch : Switches)
			{
				if (ParsedSwitch.StartsWith(TEXT("RUN=")))
				{
					FString LocalToken = ParsedSwitch.RightChop(4);
					LocalToken.TrimStartAndEndInline();
					if (!LocalToken.EndsWith(TEXT("Commandlet")))
					{
						LocalToken += TEXT("Commandlet");
					}
					SetIsRunningAsCommandlet(LocalToken);
					break;
				}
			}
		}
	}
#endif // UE_EDITOR || WITH_ENGINE || WITH_EDITOR
#if WITH_EDITOR
	int32 MultiprocessId;
	if (FParse::Value(CmdLine, TEXT("-MultiprocessId="), MultiprocessId))
	{
		UE::Private::SetMultiprocessId(MultiprocessId);
	}
#endif


	// In the commandlet case, we have set the Token to something other than the first token from commandline.
	// In all other cases we should check for the first token being the Project Specifier
	if (!bHasCommandletToken)
	{
		if (GameStringType != EGameStringType::Unknown)
		{
			// Set a new command-line that doesn't include the game name as the first argument.
			FCommandLine::Set(*FString::Join(QuotedTokenArray, TEXT(" ")));

			// Remove spurious project file tokens (which can happen on some platforms that combine commandlines).
			// This handles extra .uprojects, but if you run with MyGame MyGame, we can't tell if the second MyGame is a map or not.
			TokenArray.RemoveAll(
				[](const FString& Token)
				{
					return Token[0] != TCHAR('-') && FPaths::GetExtension(Token) == FProjectDescriptor::GetExtension();
				}
			);

			if (GameStringType == EGameStringType::ProjectPath || GameStringType == EGameStringType::ProjectShortName)
			{
				// Convert it to relative if possible...
				FString RelativeGameProjectFilePath = FFileManagerGeneric::DefaultConvertToRelativePath(*FPaths::GetProjectFilePath());
				if (RelativeGameProjectFilePath != FPaths::GetProjectFilePath())
				{
					FPaths::SetProjectFilePath(RelativeGameProjectFilePath);
				}
			}
		}

#if UE_EDITOR
		// Handle the first token being '-game' or '-server'
		if (!TokenArray.IsEmpty())
		{
			if (TokenArray[0] == TEXT("-GAME") || TokenArray[0] == TEXT("-SERVER"))
			{
				// This isn't necessarily pretty, but many requests have been made to allow
				//   UnrealEditor.exe <GAMENAME> -game <map>
				// or
				//   UnrealEditor.exe <GAMENAME> -game 127.0.0.0
				// We don't want to remove the -game from the commandline just yet in case
				// we need it for something later. So, just move it to the end for now...
				FString LocalToken = TokenArray[0];
				TokenArray.Add(LocalToken);
				TokenArray.RemoveAt(0);
				QuotedTokenArray.Add(LocalToken);
				QuotedTokenArray.RemoveAt(0);

				FCommandLine::Set(*FString::Join(QuotedTokenArray, TEXT(" ")));
			}
		}
#endif
	}

	// If we have no Commandlet or -server, test if we are running as the editor, and set the GIsEditor/GIsClient/GIsServer flags if so.
	// Do this early (and certainly before AppInit) so plugin-consumed library code can know (they wouldn't otherwise)
	// Things like the config system behave differently based on these globals, and we aren't the editor by default.
	if (!IsModeSelected())
	{
#if UE_EDITOR
		if (!Switches.Contains(TEXT("GAME")))
		{
			SetIsRunningAsEditor();
		}
#elif WITH_ENGINE && WITH_EDITOR && WITH_EDITORONLY_DATA 
		// If a non-editor target build w/ WITH_EDITOR and WITH_EDITORONLY_DATA, use the old token check...
		//@todo. Is this something we need to support?
		if (TokenArray.Contains(TEXT("EDITOR")))
		{
			SetIsRunningAsEditor();
		}
#endif
		// Game, server and non-engine programs never run as the editor
	}

#if UE_EDITOR
	// In the UE_EDITOR configuration we now know enough to finish the mode decision and decide between commandlet or client
	// In other configurations we still may need to check the Token for whether it is a commandlet and we make the decision below
	if (!IsModeSelected())
	{
		SetIsRunningAsRegularClient();
	}

	if (bHasEditorToken && GIsGameAgnosticExe)
	{
		// If we launched the editor IDE (e.g. not commandlet or -game) without a game name or project name, try to load the most recently loaded project file.
		// We can not do this if we are using a FilePlatform override since the game directory may already be established.
		const bool bIsBuildMachine = FParse::Param(FCommandLine::Get(), TEXT("BUILDMACHINE"));
		const bool bLoadMostRecentProjectFileIfItExists = !FApp::HasProjectName() && !bFileOverrideFound && !bIsBuildMachine && !FParse::Param(CmdLine, TEXT("norecentproject"));
		if (bLoadMostRecentProjectFileIfItExists)
		{
			LaunchUpdateMostRecentProjectFile();
		}
	}
#endif

#if !UE_BUILD_SHIPPING
	// Benchmarking.
	FApp::SetBenchmarking(FParse::Param(FCommandLine::Get(), TEXT("BENCHMARK")));
#else
	FApp::SetBenchmarking(false);
#endif // !UE_BUILD_SHIPPING

#if WITH_FIXED_TIME_STEP_SUPPORT
	// "-Deterministic" is a shortcut for "-UseFixedTimeStep -FixedSeed"
	bool bDeterministic = FParse::Param(FCommandLine::Get(), TEXT("Deterministic"));

	FApp::SetUseFixedTimeStep(bDeterministic || FParse::Param(FCommandLine::Get(), TEXT("UseFixedTimeStep")));

	FApp::bUseFixedSeed = bDeterministic || FApp::IsBenchmarking() || FParse::Param(FCommandLine::Get(), TEXT("FixedSeed"));
#endif

	// Initialize random number generator.
	{
		uint32 Seed1 = 0;
		uint32 Seed2 = 0;

		if (!FApp::bUseFixedSeed)
		{
			Seed1 = FPlatformTime::Cycles();
			Seed2 = FPlatformTime::Cycles();
		}

		FMath::RandInit(Seed1);
		FMath::SRandInit(Seed2);

		UE_LOGF(LogInit, Verbose, "RandInit(%d) SRandInit(%d).", Seed1, Seed2);
	}

#if !IS_PROGRAM
	if (!GIsGameAgnosticExe && FApp::HasProjectName() && !FPaths::IsProjectFilePathSet())
	{
		// If we are using a non-agnostic exe where a name was specified but we did not specify a project path. Assemble one based on the game name.
		const FString ProjectFilePath = FPaths::Combine(*FPaths::ProjectDir(), *FString::Printf(TEXT("%s.%s"), FApp::GetProjectName(), *FProjectDescriptor::GetExtension()));
		FPaths::SetProjectFilePath(ProjectFilePath);
	}
#endif

	// Initialize platform file with knowledge of the project file path before fixing the casing
	IPlatformFile* CurrentPlatformFile = &FPlatformFileManager::Get().GetPlatformFile();

	{
		SCOPED_BOOT_TIMING("PlatformFileChainElement->InitializeAfterProjectFilePath");
		for (IPlatformFile* PlatformFileChainElement = CurrentPlatformFile; PlatformFileChainElement; PlatformFileChainElement = PlatformFileChainElement->GetLowerLevel())
		{
			PlatformFileChainElement->InitializeAfterProjectFilePath();
		}
	}

#if !IS_PROGRAM
	// Now let the platform file fix the project file path case before we attempt to fix the game name
	LaunchFixProjectPathCase();
#endif

	// Now verify the project file if we have one
	if (FPaths::IsProjectFilePathSet()
#if IS_PROGRAM
		// Programs don't need uproject files to exist, but some do specify them and if they exist we should load them
		&& FPaths::FileExists(FPaths::GetProjectFilePath())
#endif
		)
	{
		SCOPED_BOOT_TIMING("IProjectManager::Get().LoadProjectFile");

		if (!IProjectManager::Get().LoadProjectFile(FPaths::GetProjectFilePath()))
		{
			// The project file was invalid or saved with a newer version of the engine. Exit.
			UE_LOGF(LogInit, Warning, "Could not find a valid project file, the engine will exit now.");
			// We will crash at some point. Try to reach log initialization to record the buffered logs and then exit.
			bErrorExitAfterSettingUpLogging = true;
		}

		if (IProjectManager::Get().IsEnterpriseProject() && FPaths::DirectoryExists(FPaths::EnterpriseDir()))
		{
			// Add the enterprise binaries directory if we're an enterprise project
			FModuleManager::Get().AddBinariesDirectory(*FPaths::Combine(FPaths::EnterpriseDir(), TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory()), false);
		}

		if (!ReplacementProjectModuleName.IsEmpty())
		{
			IProjectManager::Get().SubstituteModule(OriginalProjectModuleName, ReplacementProjectModuleName);
		}
	}

#if !IS_PROGRAM
	if (FApp::HasProjectName())
	{
		// Tell the module manager what the game binaries folder is
		FString ProjectBinariesDirectory = FPaths::Combine(ProjectBinariesRootDirectory, TEXT("Binaries"), FPlatformProcess::GetBinariesSubdirectory());
		FPlatformProcess::AddDllDirectory(*ProjectBinariesDirectory);
		FModuleManager::Get().SetGameBinariesDirectory(*ProjectBinariesDirectory);

		LaunchFixGameNameCase();
	}
#endif

#if WITH_ENGINE
	// Add the default engine shader dir
	AddShaderSourceDirectoryMapping(TEXT("/Engine"), FPlatformProcess::ShaderDir());

	// Add <PROJECT>/Shaders/ folder to virtual shader include directories
	const FString ProjectShaderPath = FPaths::Combine(FPaths::ProjectDir(), TEXT("Shaders"));
	if (FPaths::DirectoryExists(ProjectShaderPath))
	{
		AddShaderSourceDirectoryMapping(TEXT("/Project"), ProjectShaderPath);
	}

#if WITH_EDITOR
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		FString ProjectIntermediateDir = FPaths::ProjectIntermediateDir();
		bool bCreateIntermediateSuccess = PlatformFile.CreateDirectoryTree(*ProjectIntermediateDir);
		if (!bCreateIntermediateSuccess)
		{
			UE_LOGF(LogInit, Fatal, "Failed to create Intermediate directory '%ls'.", *ProjectIntermediateDir);
		}

		FString AutogenAbsolutePath = FPaths::ConvertRelativePathToFull(ProjectIntermediateDir / TEXT("ShaderAutogen"));
		bool bCreateAutogenSuccess = PlatformFile.CreateDirectory(*AutogenAbsolutePath);
		if (!bCreateAutogenSuccess)
		{
			UE_LOGF(LogInit, Fatal, "Failed to create Intermediate/ShaderAutogen/ directory '%ls'. Make sure Intermediate exists.", *AutogenAbsolutePath);
		}

		AddShaderSourceDirectoryMapping(TEXT("/ShaderAutogen"), AutogenAbsolutePath);
	}
#endif //WITH_EDITOR
#endif //WITH_ENGINE

	// Some programs might not use the taskgraph or thread pool
	bool bCreateTaskGraphAndThreadPools = true;
	// If STATS is defined (via FORCE_USE_STATS or other), we have to call FTaskGraphInterface::Startup()
#if IS_PROGRAM && !STATS
	bCreateTaskGraphAndThreadPools = !FParse::Param(FCommandLine::Get(), TEXT("ReduceThreadUsage"));
#endif
	if (bCreateTaskGraphAndThreadPools)
	{
		// initialize task graph sub-system with potential multiple threads
		SCOPED_BOOT_TIMING("FTaskGraphInterface::Startup");
		FTaskGraphInterface::Startup(FPlatformMisc::NumberOfWorkerThreadsToSpawn());
		FTaskGraphInterface::Get().AttachToThread(ENamedThreads::GameThread);
	}

#if WITH_EDITOR && PLATFORM_WINDOWS
	FWindowsPlatformPerfCounters::Init();
#endif

	if (FPlatformProcess::SupportsMultithreading() && bCreateTaskGraphAndThreadPools)
	{
		SCOPED_BOOT_TIMING("Init FQueuedThreadPool's");

		int32 StackSize = FPlatformProcess::GetDefaultPoolThreadStackSize();

		bool bForceEditorStackSize = false;
#if WITH_EDITOR
		bForceEditorStackSize = true;
#endif

		if (bHasEditorToken || bForceEditorStackSize)
		{
			StackSize = 1024 * 1024;
		}

#if WITH_EDITOR
		{
			int32 NumThreadsInThreadPool = FPlatformMisc::NumberOfWorkerThreadsToSpawn();
			// when we are in the editor we like to do things like build lighting and such
			// this thread pool can be used for those purposes
			GLargeThreadPool = new FQueuedLowLevelThreadPool();
		
			// we are only going to give dedicated servers one pool thread
			if (FPlatformProperties::IsServerOnly())
			{
				NumThreadsInThreadPool = 1;
			}

			// GThreadPool will schedule on the LargeThreadPool but limit max concurrency to the given number.
			GThreadPool = new FQueuedThreadPoolWrapper(GLargeThreadPool, NumThreadsInThreadPool);
		}
#else
		{
			GThreadPool = new FQueuedLowLevelThreadPool();
		}
#endif
		{
			GBackgroundPriorityThreadPool = FQueuedThreadPool::Allocate();
			int32 NumThreadsInThreadPool = 2;
			if (FPlatformProperties::IsServerOnly())
			{
				NumThreadsInThreadPool = 1;
			}

			verify(GBackgroundPriorityThreadPool->Create(NumThreadsInThreadPool, StackSize, TPri_Lowest, TEXT("BackgroundThreadPool")));
		}
	}

	// this can start using TaskGraph and ThreadPool so they must be created before
	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::TaskGraphSystemReady);

#if !IS_PROGRAM && !IS_MONOLITHIC
	if (FApp::HasProjectName() && FCString::Strcmp(CurrentPlatformFile->GetName(), TEXT("PakFile")) == 0)
	{
		IPluginManager::Get().SetBinariesRootDirectories(EngineBinariesRootDirectory, ProjectBinariesRootDirectory);
		IPluginManager::Get().SetPreloadBinaries();
	}
#endif

	UE::Stats::FStats::Init();
#if STATS
	FThreadStats::StartThread();
#endif

	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::StatSystemReady);

	FScopeCycleCounter CycleCount_AfterStats(GET_STATID(STAT_FEngineLoop_PreInitPreStartupScreen_AfterStats));

	// Load Core modules required for everything else to work (needs to be loaded before InitializeRenderingCVarsCaching)
	{
		SCOPED_BOOT_TIMING("LoadCoreModules");
		if (!LoadCoreModules())
		{
			UE_LOGF(LogInit, Error, "Failed to load Core modules.");
			// We will crash at some point. Try to reach log initialization to record the buffered logs and then exit.
			bErrorExitAfterSettingUpLogging = true;
		}
	}

	const bool bDumpEarlyConfigReads = FParse::Param(FCommandLine::Get(), TEXT("DumpEarlyConfigReads"));
	const bool bDumpEarlyPakFileReads = FParse::Param(FCommandLine::Get(), TEXT("DumpEarlyPakFileReads"));
	const bool bForceQuitAfterEarlyReads = FParse::Param(FCommandLine::Get(), TEXT("ForceQuitAfterEarlyReads"));

	// Overly verbose to avoid a dumb static analysis warning
#if WITH_CONFIG_PATCHING
	constexpr bool bWithConfigPatching = true;
#else
	constexpr bool bWithConfigPatching = false;
#endif

	if (bDumpEarlyConfigReads)
	{
		UE::ConfigUtilities::RecordConfigReadsFromIni();
	}

	if (bDumpEarlyPakFileReads)
	{
		RecordFileReadsFromPaks();
	}

	if(bWithConfigPatching)
	{
		UE_LOGF(LogInit, Verbose, "Begin recording CVar changes for config patching.");

		UE::ConfigUtilities::RecordApplyCVarSettingsFromIni();
	}

#if WITH_ENGINE
	extern ENGINE_API void InitializeRenderingCVarsCaching();
	InitializeRenderingCVarsCaching();
#endif

#if WITH_EDITOR && WITH_ENGINE
	// If we're running as a game or server but don't have a project, inform the user and exit.
	if (bHasEditorToken == false && bHasCommandletToken == false)
	{
		if (!FPaths::IsProjectFilePathSet())
		{
			//@todo this is too early to localize
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("Engine", "UERequiresProjectFiles", "Unreal Engine games require a project file as the first parameter."));
			// We will crash at some point. Try to reach log initialization to record the buffered logs and then exit.
			bErrorExitAfterSettingUpLogging = true;
		}
	}

#endif //WITH_EDITOR

#if WITH_APPLICATION_CORE
	// Get a pointer to the log output device
	GLogConsole = GScopedLogConsole.Get();
#endif

	// init Oodle here ; note Decompressions are okay to run before this point
	FOodleDataCompression::StartupPreInit();
	
	// install named compression format modular features
	// must be done after FCommandLine and DLL dirs are set up
	FCompression::Init();

	{
		SCOPED_BOOT_TIMING("LoadPreInitModules");
		LoadPreInitModules();
	}

#if WITH_ENGINE && CSV_PROFILER
	FCsvProfiler::Get()->Init();
#endif

#if WITH_ENGINE
	AppLifetimeEventCapture::Init();

	if (bHasEditorToken)
	{
#if WITH_EDITOR
		GWarn = &UnrealEdWarn;
#else //WITH_EDITOR
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("Engine", "EditorNotSupported", "Editor not supported in this mode."));
		FPlatformMisc::RequestExit(false, TEXT("FEngineLoop::PreInitPreStartupScreen.bHasEditorToken"));
		// We will crash at some point. Try to reach log initialization to record the buffered logs and then exit.
		bErrorExitAfterSettingUpLogging = true;
#endif //WITH_EDITOR
	}
#endif // WITH_ENGINE

#if WITH_ENGINE && FRAMEPRO_ENABLED
	FFrameProProfiler::Initialize();
#endif // FRAMEPRO_ENABLED

	// Start the application
	{
		SCOPED_BOOT_TIMING("AppInit");
		if (bErrorExitAfterSettingUpLogging)
		{
			AppInitLoggingOnly();
			return 1;
		}
		else
		{
			if (!AppInit())
			{
				return 1;
			}
		}
	}

	// Try to start the dedicated primary thread now that the command line is available,
	// and the default output devices have been created. Initializing earlier will cause
	// logs to be missed by the default output devices.
	if (!FParse::Param(FCommandLine::Get(), TEXT("NoLogThread")))
	{
		GLog->TryStartDedicatedPrimaryThread();
	}

	if (FPlatformProcess::SupportsMultithreading())
	{
		{
			SCOPED_BOOT_TIMING("GIOThreadPool->Create");
			GIOThreadPool = FQueuedThreadPool::Allocate();
			int32 NumThreadsInThreadPool = FPlatformMisc::NumberOfIOWorkerThreadsToSpawn();
			if (FPlatformProperties::IsServerOnly())
			{
				NumThreadsInThreadPool = 2;
			}
			verify(GIOThreadPool->Create(NumThreadsInThreadPool, 96 * 1024, TPri_AboveNormal, TEXT("IOThreadPool")));
		}
	}

	FEmbeddedCommunication::ForceTick(1);

#if WITH_ENGINE
	{
		SCOPED_BOOT_TIMING("System settings and cvar init");
		// Initialize system settings before anyone tries to use it...
		GSystemSettings.Initialize(bHasEditorToken);

		// Apply renderer settings from console variables stored in the INI.
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/Engine.RendererSettings"), *GEngineIni, ECVF_SetByProjectSetting);
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/Engine.RendererOverrideSettings"), *GEngineIni, ECVF_SetByProjectSetting);
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/Engine.StreamingSettings"), *GEngineIni, ECVF_SetByProjectSetting);
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/Engine.GarbageCollectionSettings"), *GEngineIni, ECVF_SetByProjectSetting);
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/Engine.NetworkSettings"), *GEngineIni, ECVF_SetByProjectSetting);
#if WITH_EDITOR
		UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/UnrealEd.CookerSettings"), *GEngineIni, ECVF_SetByProjectSetting);
#endif // WITH_EDITOR

#if !UE_SERVER
		if (!bIsRunningAsDedicatedServer)
		{
			if (!bHasCommandletToken)
			{
				// Note: It is critical that resolution settings are loaded before the movie starts playing so that the window size and fullscreen state is known
				UGameUserSettings::PreloadResolutionSettings();
			}
		}
#endif // !UE_SERVER
	}
	{
		{
			SCOPED_BOOT_TIMING("InitScalabilitySystem");
			// Init scalability system and defaults
			Scalability::InitScalabilitySystem();
		}

#if UE_WITH_CONSTINIT_UOBJECT
		{
			SCOPED_BOOT_TIMING("ConstructCompiledInObjects Early");
			UE::CoreUObject::ConstructCompiledInObjects();
		}
#endif // UE_WITH_CONSTINIT_UOBJECT

		{
			SCOPED_BOOT_TIMING("InitializeCVarsForActiveDeviceProfile");
			// Set all CVars which have been setup in the device profiles.
			// This may include scalability group settings which will override
			// the defaults set above which can then be replaced below when
			// the game user settings are loaded and applied.
			UDeviceProfileManager::InitializeCVarsForActiveDeviceProfile();
		}

		{
			SCOPED_BOOT_TIMING("Scalability::LoadState");
			// As early as possible to avoid expensive re-init of subsystems,
			// after SystemSettings.ini file loading so we get the right state,
			// before ConsoleVariables.ini so the local developer can always override.
			// after InitializeCVarsForActiveDeviceProfile() so the user can override platform defaults
			Scalability::LoadState((bHasEditorToken && !GEditorSettingsIni.IsEmpty()) ? GEditorSettingsIni : GGameUserSettingsIni);
		}

		if (FPlatformMisc::UseRenderThread())
		{
			GUseThreadedRendering = true;
		}
	}
#endif // WITH_ENGINE

	{
		SCOPED_BOOT_TIMING("LoadConsoleVariablesFromINI");
		FConfigCacheIni::LoadConsoleVariablesFromINI();
	}

	{
		SCOPED_BOOT_TIMING("Platform Initialization");
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Platform Initialization"), STAT_PlatformInit, STATGROUP_LoadTime);

		// platform specific initialization now that the SystemSettings are loaded
		FPlatformMisc::PlatformInit();
#if WITH_APPLICATION_CORE
		FPlatformApplicationMisc::Init();
#endif
		FPlatformMemory::Init();
	}

	{
		SCOPED_BOOT_TIMING("AutoRTFM");
#if WITH_VERSE_VM
		AutoRTFM::InitializeForUE(Verse::RollbackAutoRTFMWrite);
#else
		AutoRTFM::InitializeForUE(nullptr);
#endif
	}

#if WITH_ENGINE
	{
		SCOPED_BOOT_TIMING("InitDerivedData");
		UE::DerivedData::IoStore::InitializeIoDispatcher();
	}
#endif // WITH_ENGINE

#if !UE_BUILD_SHIPPING
	{
		int32 ExtraDevelopmentMemoryMB = (int32)(FPlatformMemory::GetExtraDevelopmentMemorySize() / 1024ull / 1024ull);
		CSV_METADATA(TEXT("ExtraDevelopmentMemoryMB"), *FString::FromInt(ExtraDevelopmentMemoryMB));
	}
#endif

#if USE_IO_DISPATCHER
	{
		SCOPED_BOOT_TIMING("InitIoDispatcher");
		FIoDispatcher::InitializePostSettings();
	}

	FModuleManager::Get().LoadModule("IoStoreOnDemandCore");
#if UE_WITH_IOSTOREONDEMAND
	FModuleManager::Get().LoadModule("IoStoreOnDemand");
#endif // UE_WITH_IOSTOREONDEMAND

#if UE_CLIENT || UE_EDITOR
	FModuleManager::Get().LoadModule("BandwidthDebugDelegates");
	FModuleManager::Get().LoadModule("BandwidthMeasurementTool");
	FModuleManager::Get().LoadModule("DownlinkBandwidthManager");
#endif // UE_CLIENT || UE_EDITOR

#endif // USE_IO_DISPATCHER

	// Let LogConsole know what ini file it should use to save its setting on exit.
	// We can't use GGameIni inside log console because it's destroyed in the global
	// scoped pointer and at that moment GGameIni may already be gone.
	if (GLogConsole != nullptr)
	{
		GLogConsole->SetIniFilename(*GGameIni);
	}


#if CHECK_PUREVIRTUALS
	FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("Engine", "Error_PureVirtualsEnabled", "The game cannot run with CHECK_PUREVIRTUALS enabled.  Please disable CHECK_PUREVIRTUALS and rebuild the executable."));
	FPlatformMisc::RequestExit(false, TEXT("FEngineLoop::PreInitPreStartupScreen.Check_PureVirtuals"));
#endif

	FEmbeddedCommunication::ForceTick(2);

	PreInitContext.SlowTaskPtr = new FScopedSlowTask(100, NSLOCTEXT("EngineLoop", "EngineLoop_Initializing", "Initializing..."));
	FScopedSlowTask& SlowTask = *PreInitContext.SlowTaskPtr;

#if WITH_ENGINE
	// allow for game explorer processing (including parental controls) and firewalls installation
	if (!FPlatformMisc::CommandLineCommands())
	{
		FPlatformMisc::RequestExit(false, TEXT("FEngineLoop::PreInitPreStartupScreen.CommandLineCommands"));
	}

#if !UE_EDITOR
	// UE_EDITOR doesn't care the meaning of the token except for a few special cases (FooCommandlet, -run=, -game, -server)
	// But when running without UE_EDITOR, we look up the token as a class to see if it is a commandlet name prefix
	FString LateCommandletName;
	if (!IsModeSelected())
	{
		//@hack: We need to set these before calling StaticLoadClass so all required data gets loaded for the commandlets.
		TGuardValue<bool> ScopedGIsClient(GIsClient, true);
		TGuardValue<bool> ScopedGIsServer(GIsServer, true);
#if WITH_EDITOR
		TGuardValue<bool> ScopedGIsEditor(GIsEditor, true);
#endif // WITH_EDITOR
		TGuardValue<bool> ScopedGIsRunningCommandlet(PRIVATE_GIsRunningCommandlet, true);

		// Take the first non-switch command-line parameter (i.e. one not starting with a dash). We will check if it's possibly a commandlet.
		const FString* NonSwitchParam = TokenArray.FindByPredicate(
			[](const FString& Token)
			{
				return Token[0] != TCHAR('-');
			}
		);

		bool bIsPossiblyCommandletName = NonSwitchParam != nullptr;
		if (bIsPossiblyCommandletName)
		{
			FString CommandletName = *NonSwitchParam;

			if (!CommandletName.EndsWith(TEXT("Commandlet")))
			{
				CommandletName += TEXT("Commandlet");
			}

			UClass* TempCommandletClass = FindFirstObject<UClass>(*CommandletName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Looking for commandlet class"));
			if (TempCommandletClass)
			{
				checkf(TempCommandletClass->IsChildOf(UCommandlet::StaticClass()), TEXT("It is not valid to have a class that ends with \"Commandlet\" that is not a UCommandlet subclass."));

				LateCommandletName = CommandletName;
			}
		}
	}
	if (!LateCommandletName.IsEmpty())
	{
		SetIsRunningAsCommandlet(LateCommandletName);
	}

	// In non-UE_EDITOR configuration this is the point where know enough to finish the mode decision and decide between commandlet or client
	if (!IsModeSelected())
	{
		SetIsRunningAsRegularClient();
	}
#endif

	// If std out device hasn't been initialized yet (there was no -stdout param in the command line) and
	// we meet all the criteria, initialize it now.
	if (!GScopedStdOut && !bHasEditorToken && !bIsRegularClient && !bIsRunningAsDedicatedServer)
	{
		SCOPED_BOOT_TIMING("InitializeStdOutDevice");

		InitializeStdOutDevice();
	}

#if WITH_EDITOR
	if (IsRunningCookCommandlet())
	{
		ITargetPlatformManagerModule* TargetPlatformManager = GetTargetPlatformManager(false);
		FString InitErrors;
		if (TargetPlatformManager && TargetPlatformManager->HasInitErrors(&InitErrors))
		{
			RequestEngineExit(InitErrors);
			return 1;
		}
	}
#endif

	{
		SCOPED_BOOT_TIMING("IPlatformFeaturesModule::Get()");
		// allow the platform to start up any features it may need
		IPlatformFeaturesModule::Get();
	}

	{
		SCOPED_BOOT_TIMING("InitGamePhys");
		// Init physics engine before loading anything, in case we want to do things like cook during post-load.
		if (!InitGamePhys())
		{
			// If we failed to initialize physics we cannot continue.
			return 1;
		}
	}

	{
		bool bShouldCleanShaderWorkingDirectory = true;

		// Only clean the shader working directory if we are the first instance, to avoid deleting files in use by other instances
		//@todo - check if any other instances are running right now
		bShouldCleanShaderWorkingDirectory = FPlatformProcess::IsFirstInstance();

		if (bShouldCleanShaderWorkingDirectory && !FParse::Param(FCommandLine::Get(), TEXT("Multiprocess")))
		{
			SCOPED_BOOT_TIMING("FPlatformProcess::CleanShaderWorkingDirectory");

			// get shader path, and convert it to the userdirectory
			for (const auto& SHaderSourceDirectoryEntry : AllShaderSourceDirectoryMappings())
			{
				FString ShaderDir = FString(FPlatformProcess::BaseDir()) / SHaderSourceDirectoryEntry.Value;
				FString UserShaderDir = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*ShaderDir);
				FPaths::CollapseRelativeDirectories(ShaderDir);

				// make sure we don't delete from the source directory
				if (ShaderDir != UserShaderDir)
				{
					IFileManager::Get().DeleteDirectory(*UserShaderDir, false, true);
				}
			}

			FPlatformProcess::CleanShaderWorkingDir();
		}
	}

#if !UE_BUILD_SHIPPING
	GIsDemoMode = FParse::Param(FCommandLine::Get(), TEXT("DEMOMODE"));
#endif

	// InitEngineTextLocalization loads Paks, needs Oodle to be setup before here
	InitEngineTextLocalization();

	bool bForceEnableHighDPI = false;
	// This must be called before any window (including the splash screen is created
	FSlateApplication::InitHighDPI(bForceEnableHighDPI);

	UStringTable::InitializeEngineBridge();

	if (FApp::ShouldUseThreadingForPerformance() && FPlatformMisc::AllowAudioThread())
	{
		bool bUseThreadedAudio = false;
		if (!GIsEditor)
		{
			GConfig->GetBool(TEXT("Audio"), TEXT("UseAudioThread"), bUseThreadedAudio, GEngineIni);
		}
		FAudioThread::SetUseThreadedAudio(bUseThreadedAudio);
	}

	// Ensure engine localization has loaded before we show the splash
	FTextLocalizationManager::Get().WaitForAsyncTasks();

	// Are we creating a slate application?
	bool bSlateApplication = !IsRunningDedicatedServer() && (bIsRegularClient || bHasEditorToken);
	if (bSlateApplication)
	{
		if (FPlatformProcess::SupportsMultithreading() && !FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")))
		{
			SCOPED_BOOT_TIMING("FPlatformSplash::Show()");
			FPlatformSplash::Show();
		}

		// Init platform application
		SCOPED_BOOT_TIMING("FSlateApplication::Create()");
		FSlateApplication::Create();
	}
	else
	{
		// If we're not creating the slate application there is some basic initialization
		// that it does that still must be done
		EKeys::Initialize();
		FSlateApplication::InitializeCoreStyle();
	}

	if (GIsEditor)
	{
		// The editor makes use of all cultures in its UI, so pre-load the resource data now to avoid a hitch later
		FInternationalization::Get().LoadAllCultureData();
	}

	FEmbeddedCommunication::ForceTick(3);

	SlowTask.EnterProgressFrame(5);

#if USE_LOCALIZED_PACKAGE_CACHE
	FPackageLocalizationManager::Get().InitializeFromLazyCallback([](FPackageLocalizationManager& InPackageLocalizationManager)
	{
		InPackageLocalizationManager.InitializeFromCache(MakeShareable(new FEnginePackageLocalizationCache()));
	});
#endif	// USE_LOCALIZED_PACKAGE_CACHE

	{
		SCOPED_BOOT_TIMING("FShaderParametersMetadataRegistration::CommitAll()");
		FShaderParametersMetadataRegistration::CommitAll();
	}
	
	{
		SCOPED_BOOT_TIMING("FShaderTypeRegistration::CommitAll()");
		FShaderTypeRegistration::CommitAll();
	}

	{
		SCOPED_BOOT_TIMING("FUniformBufferStruct::InitializeStructs()");
		FShaderParametersMetadata::InitializeAllUniformBufferStructs();
	}

	{
		SCOPED_BOOT_TIMING("PreInitHMDDevice()");
		PreInitHMDDevice();
	}

	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::PreRHIInit);
	SlowTask.EnterProgressFrame(5, LOCTEXT("InitializingRHI", "Initializing Render Hardware Interface..."));
	{
		SCOPED_BOOT_TIMING("RHIInit");
		// Initialize the RHI.
		RHIInit(bHasEditorToken);
	}

	{
		SCOPED_BOOT_TIMING("PipelineStateCacheInit");
		PipelineStateCache::Init();
	}

	{
		UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing Render Settings"));
		SCOPED_BOOT_TIMING("RenderUtilsInit");
		// One-time initialization of global variables based on engine configuration.
		RenderUtilsInit();
	}

	{
		bool bUseCodeLibrary = FPlatformProperties::RequiresCookedData() || GAllowCookedDataInEditorBuilds;
		if (bUseCodeLibrary)
		{
			{
				SCOPED_BOOT_TIMING("FShaderCodeLibrary::InitForRuntime");
				// Will open material shader code storage if project was packaged with it
				// This only opens the Global shader library, which is always in the content dir.
				FShaderCodeLibrary::InitForRuntime(GMaxRHIShaderPlatform);
			}

	#if !UE_EDITOR
			// Cooked data only - but also requires the code library - game only
			if (FPlatformProperties::RequiresCookedData())
			{
				SCOPED_BOOT_TIMING("FShaderPipelineCache::Initialize");
				// Initialize the pipeline cache system. Opening is deferred until the manual call to
				// OpenPipelineFileCache below, after content pak's ShaderCodeLibraries are loaded.
				FShaderPipelineCache::Initialize(GMaxRHIShaderPlatform);
			}
	#endif //!UE_EDITOR
		}
	}

#if WITH_ODSC
	check(!GODSCManager);
	if (FODSCManager::IsODSCEnabled())
	{
		GODSCManager = new FODSCManager();
	}
#endif

	if (!FPlatformProperties::RequiresCookedData())
	{
#if WITH_EDITORONLY_DATA
		{
			// Ensure that DDC is initialized from the game thread.
			UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing Derived Data Cache"));
			SCOPED_BOOT_TIMING("InitDerivedData");
			UE::DerivedData::GetCache();
			UE::DerivedData::GetBuild();
			GetDerivedDataCacheRef();
		}
#endif

#if WITH_EDITOR	
		if (UE::Virtualization::ShouldInitializePreSlate())
		{
			// Explicit initialization of the virtualization system, before slate has initialized and we cannot show error dialogs 
			UE::Virtualization::Initialize(UE::Virtualization::EInitializationFlags::None);
		}
#endif //WITH_EDITOR

		check(!GDistanceFieldAsyncQueue);
		GDistanceFieldAsyncQueue = new FDistanceFieldAsyncQueue();

		check(!GCardRepresentationAsyncQueue);
		GCardRepresentationAsyncQueue = new FCardRepresentationAsyncQueue();

		if (AllowShaderCompiling())
		{
			check(!GShaderCompilerStats);
			GShaderCompilerStats = new FShaderCompilerStats();

			check(!GShaderCompilingManager);
			GShaderCompilingManager = new FShaderCompilingManager();

			// Shader hash cache is required only for shader compilation.
			InitializeShaderHashCache();
		}
		else
		{
			// create a manager, but it won't do anything internally
			GShaderCompilingManager = new FShaderCompilingManager();
		}
	}

	{
		EShaderPermutationFlags Flags = EShaderPermutationFlags::None;
		
		bool bSupportCookedEditor = false;
		GConfig->GetBool(TEXT("CookedEditorSettings"), TEXT("bSupportCookedEditor"), bSupportCookedEditor, GGameIni);
		Flags |= bSupportCookedEditor ? EShaderPermutationFlags::HasEditorOnlyData : EShaderPermutationFlags::None;
		
		const bool bShouldCompileODSCOnlyShaders = ShouldCompileODSCOnlyShaders();
		Flags |= bShouldCompileODSCOnlyShaders ? EShaderPermutationFlags::IsODSCOnly : EShaderPermutationFlags::None;

		SetAdditionalShaderPermutationFlags(Flags);
	}

	{
		SCOPED_BOOT_TIMING("GetRendererModule");
		// Cache the renderer module in the main thread so that we can safely retrieve it later from the rendering thread.
		GetRendererModule();
	}

	{
		if (AllowShaderCompiling())
		{
			UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing Shader Types"));
			SCOPED_BOOT_TIMING("InitializeShaderTypes");

#if WITH_EDITOR
			// Explicitly generate AutogenShaderHeaders.ush prior to shader type initialization
			// (since that process will load and cache this header as a sideeffect)
			FShaderCompileUtilities::GenerateBrdfHeaders(GMaxRHIShaderPlatform);
#endif

			// Initialize shader types before loading any shaders
			InitializeShaderTypes();
		}

		FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::ShaderTypesReady);

		SlowTask.EnterProgressFrame(25, LOCTEXT("CompileGlobalShaderMap", "Compiling Global Shaders..."));

		// Load the global shaders
		if (AllowGlobalShaderLoad())
		{
			LLM_SCOPE(ELLMTag::Shaders);
			SCOPED_BOOT_TIMING("CompileGlobalShaderMap");
			CompileGlobalShaderMap(false);
			if (IsEngineExitRequested())
			{
				// This means we can't continue without the global shader map.
				return 1;
			}
		}

		SlowTask.EnterProgressFrame(5);

		{
			SCOPED_BOOT_TIMING("CreateMoviePlayer");
			CreateMoviePlayer();
		}

		if (FPreLoadScreenManager::ArePreLoadScreensEnabled())
		{
			SCOPED_BOOT_TIMING("FPreLoadScreenManager::Create");
			FPreLoadScreenManager::Create();
			ensure(FPreLoadScreenManager::Get());
		}

		// If platforms support early movie playback we have to start the rendering thread much earlier
#if PLATFORM_SUPPORTS_EARLY_MOVIE_PLAYBACK
		{
			SCOPED_BOOT_TIMING("PostInitRHI");
			PostInitRHI();
		}

		InitRenderingThread();
#endif

		FEmbeddedCommunication::ForceTick(4);

		{
#if !UE_SERVER// && !UE_EDITOR
			if (!IsRunningDedicatedServer() && !IsRunningCommandlet())
			{
				TSharedPtr<FSlateRenderer> SlateRenderer = GUsingNullRHI ?
					FModuleManager::Get().LoadModuleChecked<ISlateNullRendererModule>("SlateNullRenderer").CreateSlateNullRenderer() :
					FModuleManager::Get().GetModuleChecked<ISlateRHIRendererModule>("SlateRHIRenderer").CreateSlateRHIRenderer();
				TSharedRef<FSlateRenderer> SlateRendererSharedRef = SlateRenderer.ToSharedRef();

				{
					SCOPED_BOOT_TIMING("CurrentSlateApp.InitializeRenderer");
					// If Slate is being used, initialize the renderer after RHIInit
					FSlateApplication& CurrentSlateApp = FSlateApplication::Get();
					CurrentSlateApp.InitializeRenderer(SlateRendererSharedRef);
				}

				{
					SCOPED_BOOT_TIMING("FEngineFontServices::Create");
					// Create the engine font services now that the Slate renderer is ready
					FEngineFontServices::Create();
				}

				{
					SCOPED_BOOT_TIMING("LoadModulesForProject(ELoadingPhase::PostSplashScreen)");
					// Load up all modules that need to hook into the custom splash screen
					if (!IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PostSplashScreen) || !IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostSplashScreen))
					{
						return 1;
					}
				}

				{
					SCOPED_BOOT_TIMING("PlayFirstPreLoadScreen");

					if (FPreLoadScreenManager::Get())
					{
						{
							SCOPED_BOOT_TIMING("PlayFirstPreLoadScreen - FPreLoadScreenManager::Get()->Initialize");
							// initialize and present custom splash screen
							FPreLoadScreenManager::Get()->Initialize(SlateRendererSharedRef.Get());
						}

						if (FPreLoadScreenManager::Get()->HasRegisteredPreLoadScreenType(EPreLoadScreenTypes::CustomSplashScreen))
						{
							FPreLoadScreenManager::Get()->PlayFirstPreLoadScreen(EPreLoadScreenTypes::CustomSplashScreen);
						}
					}
				}

				PreInitContext.SlateRenderer = SlateRenderer;
			}
			else
#endif // !UE_SERVER
			{
				{
					SCOPED_BOOT_TIMING("LoadModulesForProject(ELoadingPhase::PostSplashScreen)");
					// Load up all modules that need to hook into the custom splash screen
					if (!IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PostSplashScreen) || !IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostSplashScreen))
					{
						return 1;
					}
				}
			}
		}
	}
#endif // WITH_ENGINE

	// Save PreInitContext
	PreInitContext.bDumpEarlyConfigReads = bDumpEarlyConfigReads;
	PreInitContext.bDumpEarlyPakFileReads = bDumpEarlyPakFileReads;
	PreInitContext.bForceQuitAfterEarlyReads = bForceQuitAfterEarlyReads;
	PreInitContext.bWithConfigPatching = bWithConfigPatching;
	PreInitContext.bHasEditorToken = bHasEditorToken;
#if WITH_ENGINE
	PreInitContext.bIsRegularClient = bIsRegularClient;
#endif // WITH_ENGINE
	PreInitContext.bIsPossiblyUnrecognizedCommandlet = bIsRegularClient && TokenToForward.Len() && !TokenToForward.Contains(TEXT("-"));
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	PreInitContext.bTokenDoesNotHaveDash = PreInitContext.bIsPossiblyUnrecognizedCommandlet;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	PreInitContext.Token = TokenToForward;
#if UE_EDITOR || WITH_ENGINE
	PreInitContext.CommandletCommandLine = CommandletCommandLine;
#endif // UE_EDITOR || WITH_ENGINE

#if WITH_COREUOBJECT
#if (WITH_VERSE_VM || defined(__INTELLISENSE__))
	Verse::VerseVM::Startup();
#else
	verse::FExecutionContext::Create();
#endif
#endif

	return 0;
}

void ConditionallyEnsureOnCommandletErrors(int32 InNumErrors)
{
	bool bEnsureOnError = false;
	GConfig->GetBool(TEXT("Core.System"), TEXT("EnsureCommandletOnError"), bEnsureOnError, GEngineIni);
	if (bEnsureOnError)
	{
		ensureMsgf(InNumErrors == 0, TEXT("Commandlet generated %d errors!"), InNumErrors);
	}
}

int32 FEngineLoop::PreInitPostStartupScreen(const TCHAR* CmdLine)
{
	ON_SCOPE_EXIT{ FBootProfiling::OnPreInitPostStartupScreenComplete(); };
	SCOPED_BOOT_TIMING("FEngineLoop::PreInitPostStartupScreen");
	LLM_SCOPE(ELLMTag::EnginePreInitMemory);

	if (IsEngineExitRequested())
	{
		return 0;
	}

	extern bool GIsConsoleExecutable;
#if PLATFORM_WINDOWS
	/*
	Note - ImageNtHeader must be called after DbgHelp is initialized.Failure to wait before initialization will cause calls to
	SymFromAddr for monolithic exes to fail. This results in callstacks not being written to the log.
	From discussion with Microsoft it is unclear when this time is and could be related to large pdbs. At this time here is an appropriate spot.
	*/
	if (PIMAGE_NT_HEADERS NtHeaders = ImageNtHeader(GetModuleHandle(nullptr)))
	{
		GIsConsoleExecutable = (NtHeaders->OptionalHeader.Subsystem == IMAGE_SUBSYSTEM_WINDOWS_CUI);
	}
	else
	{
		GIsConsoleExecutable = (GetFileType(GetStdHandle(STD_OUTPUT_HANDLE)) == FILE_TYPE_CHAR);
	}
#endif // PLATFORM_WINDOWS

	FScopeCycleCounter CycleCount_AfterStats(GET_STATID(STAT_FEngineLoop_PreInitPostStartupScreen_AfterStats));

	// Restore PreInitContext
	bool bDumpEarlyConfigReads = PreInitContext.bDumpEarlyConfigReads;
	bool bDumpEarlyPakFileReads = PreInitContext.bDumpEarlyPakFileReads;
	bool bForceQuitAfterEarlyReads = PreInitContext.bForceQuitAfterEarlyReads;
	bool bWithConfigPatching = PreInitContext.bWithConfigPatching;
	bool bHasEditorToken = PreInitContext.bHasEditorToken;
#if WITH_ENGINE
	bool bIsRegularClient = PreInitContext.bIsRegularClient;
#endif // WITH_ENGINE
	bool bIsPossiblyUnrecognizedCommandlet = PreInitContext.bIsPossiblyUnrecognizedCommandlet;
	FString Token = PreInitContext.Token;
#if UE_EDITOR || WITH_ENGINE
	const TCHAR* CommandletCommandLine = *PreInitContext.CommandletCommandLine;
#endif // UE_EDITOR || WITH_ENGINE

	check(PreInitContext.SlowTaskPtr);
	FScopedSlowTask& SlowTask = *PreInitContext.SlowTaskPtr;

#if WITH_ENGINE
	{
		TSharedPtr<IInstallBundleManager> BundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();

#if !UE_SERVER// && !UE_EDITOR
		if (!IsRunningDedicatedServer() && !IsRunningCommandlet())
		{
			SCOPED_BOOT_TIMING("PreInitPostStartupScreen_StartupGraphics");

			TSharedPtr<FSlateRenderer> SlateRenderer = PreInitContext.SlateRenderer;
			TSharedRef<FSlateRenderer> SlateRendererSharedRef = SlateRenderer.ToSharedRef();
			
			{
				SCOPED_BOOT_TIMING("LoadModulesForProject(ELoadingPhase::PreEarlyLoadingScreen)");
				// Load up all modules that need to hook into the loading screen
				if (!IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PreEarlyLoadingScreen) || !IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreEarlyLoadingScreen))
				{
					return 1;
				}
			}

			if (BundleManager != nullptr && !BundleManager->IsNullInterface())
			{
				IInstallBundleManager::InstallBundleCompleteDelegate.AddStatic(
					&OnStartupContentMounted, bDumpEarlyConfigReads, bDumpEarlyPakFileReads, bWithConfigPatching, bForceQuitAfterEarlyReads);
			}
			// If not using the bundle manager, config will be reloaded after ESP, see below

			//If the PreLoadScreen is active, then we don't want to initialise the MoviePlayer from the same data.
			bool hasPreloadStartupMovie = false;
			//If true then we want the splash screen to be handled defaultly
			bool useDefaultSplash = true;

			UE_SCOPED_ENGINE_ACTIVITY(TEXT("Play Preload Screen"));
			SCOPED_BOOT_TIMING("PlayFirstPreLoadScreen");

			if (FPreLoadScreenManager::Get())
			{
				SCOPED_BOOT_TIMING("PlayFirstPreLoadScreen - FPreLoadScreenManager::Get()->Initialize");
				// initialize and play our first Early PreLoad Screen if one is setup
				FPreLoadScreenManager::Get()->Initialize(SlateRendererSharedRef.Get());

				//If either early/engine screen is present then the DefaultMoviePlayer should not be configured with the same data.
				hasPreloadStartupMovie = 
					FPreLoadScreenManager::Get()->HasRegisteredPreLoadScreenType(EPreLoadScreenTypes::EarlyStartupScreen) ||
					FPreLoadScreenManager::Get()->HasRegisteredPreLoadScreenType(EPreLoadScreenTypes::EngineLoadingScreen);

				if (FPreLoadScreenManager::Get()->HasRegisteredPreLoadScreenType(EPreLoadScreenTypes::EarlyStartupScreen))
				{
					// disable the splash before playing the early startup screen
					FPreLoadScreenManager::Get()->IsResponsibleForRenderingDelegate.AddLambda(
						[](bool bIsPreloadScreenManResponsibleForRendering)
						{
							FPlatformMisc::PlatformHandleSplashScreen(!bIsPreloadScreenManResponsibleForRendering);
						}
					);
					FPreLoadScreenManager::Get()->PlayFirstPreLoadScreen(EPreLoadScreenTypes::EarlyStartupScreen);

					useDefaultSplash = false;
				}
				else
				{
					useDefaultSplash = true;
				}
			}

			//If a PreLoadScreenManager is present, then MoviePlayer should not read the same configurable data and rely on a different configuration setup.
			if (IsMoviePlayerEnabled() && !hasPreloadStartupMovie)
			{
				SCOPED_BOOT_TIMING("GetMoviePlayer()->SetupLoadingScreenFromIni");
				// allow the movie player to load a sequence from the .inis (a PreLoadingScreen module could have already initialized a sequence, in which case
				// it wouldn't have anything in it's .ini file)
				GetMoviePlayer()->SetupLoadingScreenFromIni();
			}

			if (IsMoviePlayerEnabled() && GetMoviePlayer()->HasEarlyStartupMovie())
			{
				SCOPED_BOOT_TIMING("EarlyStartupMovie");
				GetMoviePlayer()->Initialize(SlateRendererSharedRef.Get(), FPreLoadScreenManager::Get() ? FPreLoadScreenManager::Get()->GetRenderWindow() : nullptr);

				// hide splash screen now before playing any movies
				FPlatformMisc::PlatformHandleSplashScreen(false);

				// only allowed to play any movies marked as early startup.  These movies or widgets can have no interaction whatsoever with uobjects or engine features
				GetMoviePlayer()->PlayEarlyStartupMovies();

				// display the splash screen again now that early startup movies have played
				FPlatformMisc::PlatformHandleSplashScreen(true);

#if 0 && PAK_TRACKER// dump the files which have been accessed inside the pak file
				FPakPlatformFile* PakPlatformFile = (FPakPlatformFile*)(FPlatformFileManager::Get().FindPlatformFile(FPakPlatformFile::GetTypeName()));
				FString FileList = "All files accessed before init\n";
				for (const auto& PakMapFile : PakPlatformFile->GetPakMap())
				{
					FileList += PakMapFile.Key + TEXT("\n");
				}
				UE_LOGF(LogInit, Display, "\n%ls", *FileList);
#endif

				useDefaultSplash &= true;
			}

			if (useDefaultSplash)
			{
				// no early startup screen, show the splash screen
				FPlatformMisc::PlatformHandleSplashScreen(true);
			}
		}
		else
#endif // !UE_SERVER
		{
#if !UE_SERVER
			if (IsRunningCommandlet())
			{
				// Create the engine font services now that the Slate renderer is ready
				FEngineFontServices::Create();
			}
#endif
			{
				SCOPED_BOOT_TIMING("LoadModulesForProject(ELoadingPhase::PreEarlyLoadingScreen)");
				// Load up all modules that need to hook into the loading screen, even though there
				// is no loading screen we still need to load the modules registered in that phase.
				if (!IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PreEarlyLoadingScreen)
					|| !IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreEarlyLoadingScreen))
				{
					return 1;
				}
			}
		}

		//Now that our EarlyStartupScreen is finished, lets take the necessary steps to mount paks, apply .ini cvars, and open the shader libraries if we installed content we expect to handle
		//If using a bundle manager, assume its handling all this stuff and that we don't have to do it.
		if (BundleManager == nullptr || BundleManager->IsNullInterface() || !BundleManager->SupportsEarlyStartupPatching())
		{
			// Mount Paks that were installed during EarlyStartupScreen
			if (FCoreDelegates::OnMountAllPakFiles.IsBound() && FPaths::HasProjectPersistentDownloadDir() )
			{
				SCOPED_BOOT_TIMING("MountPaksAfterEarlyStartupScreen");

				FString InstalledGameContentDir = FPaths::Combine(*FPaths::ProjectPersistentDownloadDir(), TEXT("InstalledContent"), FApp::GetProjectName(), TEXT("Content"), TEXT("Paks"));
				FPlatformMisc::AddAdditionalRootDirectory(FPaths::Combine(*FPaths::ProjectPersistentDownloadDir(), TEXT("InstalledContent")));

				TArray<FString> PakFolders;
				PakFolders.Add(InstalledGameContentDir);
				FCoreDelegates::OnMountAllPakFiles.Execute(PakFolders);

				// Look for any plugins installed during EarlyStartupScreen
				IPluginManager::Get().RefreshPluginsList();
				IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreEarlyLoadingScreen);
			}

			DumpEarlyReads(bDumpEarlyConfigReads, bDumpEarlyPakFileReads, bForceQuitAfterEarlyReads);

			//Reapply CVars after our EarlyLoadScreen
			if(bWithConfigPatching)
			{
				SCOPED_BOOT_TIMING("ReapplyCVarsFromIniAfterEarlyStartupScreen");
				HandleConfigReload(bWithConfigPatching);
			}

			//Handle opening shader library after our EarlyLoadScreen
			{
				LLM_SCOPE(ELLMTag::Shaders);
				SCOPED_BOOT_TIMING("FShaderCodeLibrary::OpenLibrary");

				// Open the game library which contains the material shaders.
				FShaderCodeLibrary::OpenLibrary(FApp::GetProjectName(), FPaths::ProjectContentDir());
				for (const FString& RootDir : FPlatformMisc::GetAdditionalRootDirectories())
				{
					FShaderCodeLibrary::OpenLibrary(FApp::GetProjectName(), FPaths::Combine(RootDir, FApp::GetProjectName(), TEXT("Content")));
				}

				// Now our shader code main library is opened, kick off the precompile, if already initialized
				FShaderPipelineCache::OpenPipelineFileCache(GMaxRHIShaderPlatform);
			}
		}
#if WITH_EDITOR
		else if (GAllowCookedDataInEditorBuilds)
		{
			//Handle opening shader library after our EarlyLoadScreen
			{
				LLM_SCOPE(ELLMTag::Shaders);
				SCOPED_BOOT_TIMING("FShaderCodeLibrary::OpenLibrary");

				// Open the game library which contains the material shaders.
				FShaderCodeLibrary::OpenLibrary(FApp::GetProjectName(), FPaths::ProjectContentDir());
			}
		}
#endif
		
		InitGameTextLocalization();

		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Initial UObject load"), STAT_InitialUObjectLoad, STATGROUP_LoadTime);

		// In order to be able to use short script package names get all script
		// package names from ini files and register them with FPackageName system.
		FPackageName::RegisterShortPackageNamesForUObjectModules();

		SlowTask.EnterProgressFrame(5);


		{
		    SCOPED_BOOT_TIMING("LoadAssetRegistryModule");
			UE_SCOPED_ENGINE_ACTIVITY(TEXT("Loading AssetRegistry"));
			// If we don't do this now and the async loading thread is active, then we will attempt to load this module from a thread
		    FModuleManager::Get().LoadModule("AssetRegistry");
		}
#if WITH_COREUOBJECT
		// Initialize the PackageResourceManager, which is needed to load any (non-script) Packages. It is first used in ProcessNewlyLoadedObjects (due to the loading of asset references in Class Default Objects)
		// It has to be intialized after the AssetRegistryModule; the editor implementations of PackageResourceManager relies on it
		IPackageResourceManager::Initialize();
#endif
#if WITH_EDITOR
		// Initialize the BulkDataRegistry, which registers BulkData structs loaded from Packages for later building. It uses the same lifetime as IPackageResourceManager
		IBulkDataRegistry::Initialize();
#endif

		FEmbeddedCommunication::ForceTick(5);

		// for any auto-registered functions that want to wait until main(), run them now
		// @todo loadtime: this should have phases, so caller can decide when auto-register runs [use plugin phases probably?]
		FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::PreObjectSystemReady);

		// Make sure all UObject classes are registered and default properties have been initialized
		{
			UE_SCOPED_ENGINE_ACTIVITY(TEXT("Initializing UObject Classes"));
			ProcessNewlyLoadedUObjects();
		}

#if WITH_EDITOR

		// To make sure the shader headers are generated correctly, it needs to run after ProcessNewlyLoadedUObjects. ProcessNewlyLoadedUObjects sets up
		// the DeviceProfileManager, which is necessary to access certain console variables.
		// Also, the shader header needs to be generated before the call to GEditor->InitEditor because some modules (at least Niagara) will generate
		// and cache shaders dependencies during plugin loading so I need the shader headers at that moment.
		if (AllowShaderCompiling())
		{
			GenerateShaderHeaders();
		}

		if (!UE::Virtualization::ShouldInitializePreSlate())
		{
			// Explicit initialization of the virtualization system, after slate has initialized and we can show error dialogs.
			UE::Virtualization::Initialize(UE::Virtualization::EInitializationFlags::None);
		}
#endif //WITH_EDITOR

		// Ensure game localization has loaded before we continue
		FTextLocalizationManager::Get().WaitForAsyncTasks();

		FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::ObjectSystemReady);

		FEmbeddedCommunication::ForceTick(6);

#if USE_LOCALIZED_PACKAGE_CACHE
		{
			SCOPED_BOOT_TIMING("FPackageLocalizationManager::Get().PerformLazyInitialization()");
			// CoreUObject is definitely available now, so make sure the package localization cache is available
			// This may have already been initialized from the CDO creation from ProcessNewlyLoadedUObjects
			FPackageLocalizationManager::Get().PerformLazyInitialization();
		}
#endif	// USE_LOCALIZED_PACKAGE_CACHE

		{
			SCOPED_BOOT_TIMING("InitDefaultMaterials etc");
			// Default materials may have been loaded due to dependencies when loading
			// classes and class default objects. If not, do so now.
			UMaterialInterface::InitDefaultMaterials();
			UMaterialInterface::AssertDefaultMaterialsExist();
			UMaterialInterface::AssertDefaultMaterialsPostLoaded();
		}
	}

	{
		SCOPED_BOOT_TIMING("IStreamingManager::Get()");
		// Initialize the texture streaming system (needs to happen after RHIInit and ProcessNewlyLoadedUObjects).
		IStreamingManager::Get();
	}

	SlowTask.EnterProgressFrame(5);

	// Tell the module manager is may now process newly-loaded UObjects when new C++ modules are loaded
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	FEmbeddedCommunication::ForceTick(7);

	SlowTask.EnterProgressFrame(10);

	{
		SCOPED_BOOT_TIMING("LoadStartupCoreModules");
		if (!LoadStartupCoreModules())
		{
			// At least one startup module failed to load, return 1 to indicate an error
			return 1;
		}
	}


	SlowTask.EnterProgressFrame(10);

	{
		SCOPED_BOOT_TIMING("IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PreLoadingScreen)");
		// Load up all modules that need to hook into the loading screen
		if (!IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PreLoadingScreen) || !IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreLoadingScreen))
		{
			return 1;
		}
	}

#if !UE_SERVER
    //See if we have an engine loading PreLoadScreen registered, if not try to play an engine loading movie as a backup.
    if (IsMoviePlayerEnabled() && !GetMoviePlayer()->IsMovieCurrentlyPlaying())
    {
		SCOPED_BOOT_TIMING("FPreLoadScreenManager::Get()->Initialize etc");
		if (FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer())
        {
            if (FPreLoadScreenManager::Get())
            {
                if (FPreLoadScreenManager::Get()->HasRegisteredPreLoadScreenType(EPreLoadScreenTypes::EngineLoadingScreen))
                {
                    FPreLoadScreenManager::Get()->Initialize(*Renderer);
                }
                else
                {
                    //If we don't have a PreLoadScreen to show, try and initialize old flow with the movie player.
                    GetMoviePlayer()->Initialize(*Renderer, FPreLoadScreenManager::Get()->GetRenderWindow());
                }
            }
            else
            {
                GetMoviePlayer()->Initialize(*Renderer, nullptr);
            }
        }
    }
#endif

	{
		SCOPED_BOOT_TIMING("FPlatformApplicationMisc::PostInit");
		// do any post appInit processing, before the render thread is started.
		FPlatformApplicationMisc::PostInit();
	}
	SlowTask.EnterProgressFrame(5);

#if !PLATFORM_SUPPORTS_EARLY_MOVIE_PLAYBACK
	{
		SCOPED_BOOT_TIMING("PostInitRHI etc");
		PostInitRHI();
	}

	InitRenderingThread();
#endif // !PLATFORM_SUPPORTS_EARLY_MOVIE_PLAYBACK

	// Playing a movie can only happen after the rendering thread is started.
#if !UE_SERVER// && !UE_EDITOR
	if (IsMoviePlayerEnabled() && !GetMoviePlayer()->IsMovieCurrentlyPlaying())
	{
		SCOPED_BOOT_TIMING("PlayFirstPreLoadScreen etc");
		if (FPreLoadScreenManager::Get() && FPreLoadScreenManager::Get()->HasRegisteredPreLoadScreenType(EPreLoadScreenTypes::EngineLoadingScreen))
        {
            FPreLoadScreenManager::Get()->PlayFirstPreLoadScreen(EPreLoadScreenTypes::EngineLoadingScreen);
			FPreLoadScreenManager::Get()->SetEngineLoadingComplete(false);
        }
        else
        {
            // Play any non-early startup loading movies.
            GetMoviePlayer()->PlayMovie();
        }
	}
#endif

	{
		SCOPED_BOOT_TIMING("PlatformHandleSplashScreen etc");
#if !UE_SERVER
		if (!IsRunningDedicatedServer())
		{
			// show or hide splash screen based on movie
			FPlatformMisc::PlatformHandleSplashScreen(!(IsMoviePlayerEnabled() && GetMoviePlayer()->IsMovieCurrentlyPlaying()));
		}
		else
#endif
		{
			// show splash screen
			FPlatformMisc::PlatformHandleSplashScreen(true);
		}
	}

	{
		FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy.AddStatic(StartRenderCommandFenceBundler);
		FCoreUObjectDelegates::PostGarbageCollectConditionalBeginDestroy.AddStatic(StopRenderCommandFenceBundler);
	}

#if WITH_EDITOR
	// We need to mount the shared resources for templates (if there are any) before we try and load and game classes
	FUnrealEdMisc::Get().MountTemplateSharedPaths();

	// We need to load any animation compression settings before we load game classes
	FAnimationUtils::PreloadCompressionSettings();
#endif
	
	{
		SCOPED_BOOT_TIMING("LoadStartupModules");
		if (!LoadStartupModules())
		{
			// At least one startup module failed to load, return 1 to indicate an error
			return 1;
		}
	}
#else // !WITH_ENGINE
#if WITH_COREUOBJECT
	// Initialize the PackageResourceManager, which is needed to load any (non-script) Packages.
	IPackageResourceManager::Initialize();
#endif
#endif // WITH_ENGINE

#if WITH_COREUOBJECT
	if (GUObjectArray.IsOpenForDisregardForGC())
	{
		// Flushing async loading before closing the DisregardForGC window to avoid having packages that start async loading before the window is closed
		// contain assets that are created after the window is closed.
		FlushAsyncLoading();

		SCOPED_BOOT_TIMING("CloseDisregardForGC");
		GUObjectArray.CloseDisregardForGC();
	}
	NotifyRegistrationComplete();
	FReferencerFinder::NotifyRegistrationComplete();
#endif // WITH_COREUOBJECT

#if WITH_ENGINE
	if (UOnlineEngineInterface::Get()->IsLoaded())
	{
		SetIsServerForOnlineSubsystemsDelegate(FQueryIsRunningServer::CreateStatic(&IsServerDelegateForOSS));
	}

	SlowTask.EnterProgressFrame(5);

	if (!bHasEditorToken && !IsRunningDedicatedServer())
	{
		UClass* CommandletClass = nullptr;

		if (!bIsRegularClient)
		{
			checkf(PRIVATE_GIsRunningCommandlet, TEXT("This should have been set in PreInitPreStartupScreen"));

			CommandletClass = Cast<UClass>(StaticFindFirstObject(UClass::StaticClass(), *Token, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("looking for commandlet")));
			int32 PeriodIdx;
			if (!CommandletClass && Token.FindChar('.', PeriodIdx))
			{
				// try to load module for commandlet specified before a period.
				FModuleManager::Get().LoadModule(*Token.Left(PeriodIdx));
				CommandletClass = FindFirstObject<UClass>(*Token, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Looking for commandlet class"));
			}
			if (!CommandletClass)
			{
				if (GLogConsole && !GIsSilent)
				{
					GLogConsole->Show(true);
				}
				UE_LOGF(LogInit, Error, "%ls looked like a commandlet, but we could not find the class.", *Token);
				RequestEngineExit(FString::Printf(TEXT("Failed to find commandlet class %s"), *Token));
				return 1;
			}

#if PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_UNIX
			if (GIsConsoleExecutable)
			{
				if (GLogConsole != nullptr && GLogConsole->IsAttached())
				{
					GLog->RemoveOutputDevice(GLogConsole);
				}
				// Setup Ctrl-C handler for console application
				FPlatformMisc::SetGracefulTerminationHandler();
			}
			else
#endif
			{
				// Bring up console unless we're a silent build.
				if( GLogConsole && !GIsSilent )
				{
					GLogConsole->Show( true );
				}
			}

			// print output immediately
			setvbuf(stdout, nullptr, _IONBF, 0);

			UE_LOGF(LogInit, Log,  "Executing %ls", *CommandletClass->GetFullName() );

			// Allow commandlets to individually override those settings.
			UCommandlet* Default = CastChecked<UCommandlet>(CommandletClass->GetDefaultObject());

			if ( IsEngineExitRequested() )
			{
				// commandlet set IsEngineExitRequested() during construction
				return 1;
			}

			GIsClient = Default->IsClient;
			GIsServer = Default->IsServer;
#if WITH_EDITOR
			GIsEditor = Default->IsEditor;
#else
			if (Default->IsEditor)
			{
				UE_LOGF(LogInit, Error, "Cannot run editor commandlet %ls with game executable.", *CommandletClass->GetFullName());
				RequestEngineExit(TEXT("Tried to run commandlet in non-editor build"));
				return 1;
			}
#endif
			// Reset aux log if we don't want to log to the console window.
			if( !Default->LogToConsole )
			{
				GLog->RemoveOutputDevice( GLogConsole );
			}

			// allow the commandlet the opportunity to create a custom engine
			CommandletClass->GetDefaultObject<UCommandlet>()->CreateCustomEngine(CommandletCommandLine);
			if ( GEngine == nullptr )
			{
#if WITH_EDITOR
				if ( GIsEditor )
				{
					FString EditorEngineClassName;
					GConfig->GetString(TEXT("/Script/Engine.Engine"), TEXT("EditorEngine"), EditorEngineClassName, GEngineIni);
					UClass* EditorEngineClass = StaticLoadClass( UEditorEngine::StaticClass(), nullptr, *EditorEngineClassName);
					if (EditorEngineClass == nullptr)
					{
						UE_LOGF(LogInit, Fatal, "Failed to load Editor Engine class '%ls'.", *EditorEngineClassName);
					}

					GEngine = GEditor = NewObject<UEditorEngine>(GetTransientPackage(), EditorEngineClass);

					GEngine->ParseCommandline();

					UE_LOGF(LogInit, Log, "Initializing Editor Engine...");
					GEditor->InitEditor(this);
					UE_LOGF(LogInit, Log, "Initializing Editor Engine Completed");
				}
				else
#endif
				{
					FString GameEngineClassName;
					GConfig->GetString(TEXT("/Script/Engine.Engine"), TEXT("GameEngine"), GameEngineClassName, GEngineIni);

					UClass* EngineClass = StaticLoadClass( UEngine::StaticClass(), nullptr, *GameEngineClassName);

					if (EngineClass == nullptr)
					{
						UE_LOGF(LogInit, Fatal, "Failed to load Engine class '%ls'.", *GameEngineClassName);
					}

					// must do this here so that the engine object that we create on the next line receives the correct property values
					GEngine = NewObject<UEngine>(GetTransientPackage(), EngineClass);
					check(GEngine);

					GEngine->ParseCommandline();

					UE_LOGF(LogInit, Log, "Initializing Game Engine...");
					GEngine->Init(this);
					UE_LOGF(LogInit, Log, "Initializing Game Engine Completed");
				}
			}

			// Call init callbacks
			{
				SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnPostEngineInit", CoreDelegatesColor);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FCoreDelegates::OnPostEngineInit.Broadcast();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				FCoreDelegates::GetOnPostEngineInit().Broadcast();
			}

			// Load all the post-engine init modules
			ensure(IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PostEngineInit));
			ensure(IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostEngineInit));

			// Call module loading phases completion callbacks
			SetEngineStartupModuleLoadingComplete();

			// Disable the backlog only after module loading is complete because that gives modules
			// a chance to consume the backlog when constructing output devices.
			GLog->EnableBacklog(false);

			//run automation smoke tests now that the commandlet has had a chance to override the above flags and GEngine is available
			FAutomationTestFramework::Get().RunSmokeTests();

			UCommandlet* Commandlet = NewObject<UCommandlet>(GetTransientPackage(), CommandletClass);
			check(Commandlet);
			Commandlet->AddToRoot();

			// Execute the commandlet.
			double CommandletExecutionStartTime = FPlatformTime::Seconds();

			// Commandlets don't always handle -run= properly in the commandline so we'll provide them
			// with a custom version that doesn't have it.
			Commandlet->ParseParms( CommandletCommandLine );
#if	STATS
			// We have to close the scope, otherwise we will end with broken stats.
			CycleCount_AfterStats.StopAndResetStatId();
#endif // STATS
			UE::Stats::FStats::TickCommandletStats();

#if WITH_ENGINE
			PRIVATE_GRunningCommandletClass = CommandletClass;
#endif

			{
				SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnCommandletPreMain", CoreDelegatesColor);
				FCoreDelegates::OnCommandletPreMain.Broadcast();
			}

			int32 ErrorLevel;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*WriteToString<512>(TEXT("Commandlet Main "), Commandlet->GetFName()));
				FTrackedActivityScope CommandletActivity(FTrackedActivity::GetEngineActivity(), *FString::Printf(TEXT("Running %s"), *Commandlet->GetName()), false, FTrackedActivity::ELight::Green);
				ErrorLevel = Commandlet->Main(CommandletCommandLine);
			}

			{
				SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnCommandletPostMain", CoreDelegatesColor);
				FCoreDelegates::OnCommandletPostMain.Broadcast();
			}

#if WITH_ENGINE
			PRIVATE_GRunningCommandletClass = nullptr;
#endif

			UE::Stats::FStats::TickCommandletStats();

			RequestEngineExit(FString::Printf(TEXT("Commandlet %s finished execution (result %d)"), *Commandlet->GetName(), ErrorLevel));

			// Log warning/ error summary.
			if( Commandlet->ShowErrorCount )
			{
				TArray<FString> AllErrors;
				TArray<FString> AllWarnings;
				GWarn->GetErrors(AllErrors);
				GWarn->GetWarnings(AllWarnings);

				if (AllErrors.Num() || AllWarnings.Num())
				{
					SET_WARN_COLOR(COLOR_WHITE);
					UE_LOGF(LogInit, Display, "");
					UE_LOGF(LogInit, Display, "Warning/Error Summary (Unique only)");
					UE_LOGF(LogInit, Display, "-----------------------------------");

					const int32 MaxMessagesToShow = (GIsBuildMachine || FParse::Param(FCommandLine::Get(), TEXT("DUMPALLWARNINGS"))) ?
						(AllErrors.Num() + AllWarnings.Num()) : 50;

					TSet<FString> ShownMessages;
					ShownMessages.Empty(MaxMessagesToShow);

					SET_WARN_COLOR(COLOR_RED);

					for (const FString& ErrorMessage : AllErrors)
					{
						bool bAlreadyShown = false;
						ShownMessages.Add(ErrorMessage, &bAlreadyShown);

						if (!bAlreadyShown)
						{
							if (ShownMessages.Num() > MaxMessagesToShow)
							{
								SET_WARN_COLOR(COLOR_WHITE);
								UE_CLOG(MaxMessagesToShow < AllErrors.Num(), LogInit, Display, TEXT("NOTE: Only first %d errors displayed."), MaxMessagesToShow);
								break;
							}

							UE_LOGF(LogInit, Display, "%ls", *ErrorMessage);
						}
					}

					SET_WARN_COLOR(COLOR_YELLOW);
					ShownMessages.Empty(MaxMessagesToShow);

					for (const FString& WarningMessage : AllWarnings)
					{
						bool bAlreadyShown = false;
						ShownMessages.Add(WarningMessage, &bAlreadyShown);

						if (!bAlreadyShown)
						{
							if (ShownMessages.Num() > MaxMessagesToShow)
							{
								SET_WARN_COLOR(COLOR_WHITE);
								UE_CLOG(MaxMessagesToShow < AllWarnings.Num(), LogInit, Display, TEXT("NOTE: Only first %d warnings displayed."), MaxMessagesToShow);
								break;
							}

							UE_LOGF(LogInit, Display, "%ls", *WarningMessage);
						}
					}
				}

				// Allow the caller to request that we issue an ensure when there are any errors from the executed commandlet
				ConditionallyEnsureOnCommandletErrors(AllErrors.Num());

				UE_LOGF(LogInit, Display, "");

				if( ErrorLevel != 0 )
				{
					SET_WARN_COLOR(COLOR_RED);
					UE_LOGF(LogInit, Display, "Commandlet->Main return this error code: %d", ErrorLevel );
					UE_LOGF(LogInit, Display, "With %d error(s), %d warning(s)", AllErrors.Num(), AllWarnings.Num() );
				}
				else if( ( AllErrors.Num() == 0 ) )
				{
					SET_WARN_COLOR(AllWarnings.Num() ? COLOR_YELLOW : COLOR_GREEN);
					UE_LOGF(LogInit, Display, "Success - %d error(s), %d warning(s)", AllErrors.Num(), AllWarnings.Num() );
				}
				else
				{
					SET_WARN_COLOR(COLOR_RED);
					UE_LOGF(LogInit, Display, "Failure - %d error(s), %d warning(s)", AllErrors.Num(), AllWarnings.Num() );
				}
				CLEAR_WARN_COLOR();
			}
			else
			{
				UE_LOGF(LogInit, Display, "Finished.");
			}

			// Return an non-zero code if errors were logged and UseCommandletResultAsExitCode is false
			if (!Commandlet->UseCommandletResultAsExitCode)
			{
				if ((ErrorLevel == 0) && (GWarn->GetNumErrors() > 0))
				{
					ErrorLevel = 1;
				}
			}

			double CommandletExecutionTime = FPlatformTime::Seconds() - CommandletExecutionStartTime;
			FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
			uint64 PeakPhysMemoryMB = static_cast<uint64>(Stats.PeakUsedPhysical) / (1024 * 1024);
			uint64 PeakVirtMemoryMB = static_cast<uint64>(Stats.PeakUsedVirtual) / (1024 * 1024);
			if (CommandletExecutionTime <= 60)
			{
				UE_LOG(LogInit, Display, LINE_TERMINATOR TEXT("Execution of commandlet took:  %.2f seconds, PeakPhysMemoryMB=%llu MB, PeakVirtMemoryMB=%llu MB"), 
					CommandletExecutionTime, PeakPhysMemoryMB, PeakVirtMemoryMB);
			}
			else
			{
				FTimespan ExecutionTimeSpan = FTimespan::FromSeconds(CommandletExecutionTime);
				int32 Hours = (int32)(ExecutionTimeSpan.GetTotalHours());
				int32 Minutes = ExecutionTimeSpan.GetMinutes();
				int32 Seconds = ExecutionTimeSpan.GetSeconds();
				// Tried FText::AsTimespan here but it actually felt harder to visually parse than just explicit labeling. Leaving the
				// seconds in so that it's easy to subtract between multiple runs.
				if (Hours)
				{
					UE_LOG(LogInit, Display, LINE_TERMINATOR TEXT("Execution of commandlet took:  %dh %dm %ds (%.2f seconds), PeakPhysMemoryMB=%llu MB, PeakVirtMemoryMB=%llu MB"), Hours, Minutes, Seconds, CommandletExecutionTime,
						PeakPhysMemoryMB, PeakVirtMemoryMB);
				}
				else
				{
					UE_LOG(LogInit, Display, LINE_TERMINATOR TEXT("Execution of commandlet took:  %dm %ds (%.2f seconds), PeakPhysMemoryMB=%llu MB, PeakVirtMemoryMB=%llu MB"), Minutes, Seconds, CommandletExecutionTime,
						PeakPhysMemoryMB, PeakVirtMemoryMB);
				}
			}

			const bool bShouldPerformFastExit = Commandlet->FastExit || FParse::Param(FCommandLine::Get(), TEXT("fastexit"));
			if (bShouldPerformFastExit)
			{
				FPlatformMisc::RequestExitWithStatus(true, ErrorLevel);
			}

			// We're ready to exit!
			return ErrorLevel;
		}
		else
		{
			// We're a regular client.
			check(bIsRegularClient);

			if (bIsPossiblyUnrecognizedCommandlet)
			{
				// here we give people a reasonable warning if they tried to use the short name of a commandlet
				UClass* TempCommandletClass = FindFirstObject<UClass>(*(Token + TEXT("Commandlet")), EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Looking for commandlet class"));
				if (TempCommandletClass)
				{
					UE_LOGF(LogInit, Fatal, "You probably meant to call a commandlet. Please use the full name %ls.", *(Token+TEXT("Commandlet")));
				}
			}
		}
	}

	// exit if wanted.
	if( IsEngineExitRequested() )
	{
		if ( GEngine != nullptr )
		{
#if WITH_EDITOR
			if (GIsEditor)
			{
				SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnEditorPreExit", CoreDelegatesColor);
				FEditorDelegates::OnEditorPreExit.Broadcast();
			}
#endif // WITH_EDITOR

			GEngine->PreExit();
		}
		AppPreExit();
		// appExit is called outside guarded block.
		return 1;
	}

	FEmbeddedCommunication::ForceTick(8);

	if(FParse::Param(FCommandLine::Get(),TEXT("DUMPMOVIE")))
	{
		// -1: remain on
		GIsDumpingMovie = -1;
	}

	// If dumping movie then we do NOT want on-screen messages
	GAreScreenMessagesEnabled = !GIsDumpingMovie && !GIsDemoMode;

#if !UE_BUILD_SHIPPING
	if (FParse::Param(FCommandLine::Get(),TEXT("NOSCREENMESSAGES")))
	{
		GAreScreenMessagesEnabled = false;
	}

	if (GEngine && FParse::Param(FCommandLine::Get(), TEXT("statunit")))
	{
		GEngine->Exec(nullptr, TEXT("stat unit"));
	}

	// Don't update INI files if benchmarking or -noini
	if( FApp::IsBenchmarking() || FParse::Param(FCommandLine::Get(),TEXT("NOINI")))
	{
		GConfig->Detach( GEngineIni );
		GConfig->Detach( GInputIni );
		GConfig->Detach( GGameIni );
		GConfig->Detach( GEditorIni );
	}
#endif // !UE_BUILD_SHIPPING

	// initialize the pointer, as it is deleted before being assigned in the first frame
	PendingCleanupObjects = nullptr;

	// Initialize profile visualizers.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FModuleManager::Get().LoadModule(TEXT("ProfileVisualizer"));
#endif

	// Init HighRes screenshot system, unless running on server
	if (!IsRunningDedicatedServer())
	{
		GetHighResScreenshotConfig().Init();
	}

	// precache compute PSOs for global shaders - enough time should have passed since loading global SM to avoid a blocking load
	if (AllowGlobalShaderLoad())
	{
		PrecacheComputePipelineStatesForGlobalShaders(GMaxRHIFeatureLevel, nullptr);
	}

	UE::RenderCommandPipe::Initialize();

#else // WITH_ENGINE
	InitEngineTextLocalization();
	InitGameTextLocalization();
	FTextLocalizationManager::Get().WaitForAsyncTasks();
#if USE_LOCALIZED_PACKAGE_CACHE
	{
		SCOPED_BOOT_TIMING("FPackageLocalizationManager::Get().InitializeFromDefaultCache");
		FPackageLocalizationManager::Get().InitializeFromDefaultCache();
	}
#endif	// USE_LOCALIZED_PACKAGE_CACHE
#if WITH_APPLICATION_CORE
	{
		SCOPED_BOOT_TIMING("FPlatformApplicationMisc::PostInit");
		FPlatformApplicationMisc::PostInit();
	}
#endif
#endif // WITH_ENGINE

	{
		SCOPED_BOOT_TIMING("RunSmokeTests");
		//run automation smoke tests now that everything is setup to run
		FAutomationTestFramework::Get().RunSmokeTests();
	}

#if WITH_ENGINE && (!UE_BUILD_SHIPPING)
	IRHITestModule* RHIUnitTests = static_cast<IRHITestModule*>(FModuleManager::Get().GetModule(TEXT("RHITests")));
	if (RHIUnitTests)
	{
		RHIUnitTests->RunAllTests();
	}
#endif //(!UE_BUILD_SHIPPING)

	FEmbeddedCommunication::ForceTick(9);

	PreInitContext.Cleanup();

	// Note we still have 20% remaining on the slow task: this will be used by the Editor/Engine initialization next
	return 0;
}

int32 FEngineLoop::PreInit(const TCHAR* CmdLine)
{
#if UE_ENABLE_ARRAY_SLACK_TRACKING
	// Any array allocations before this point won't have array slack tracking, although subsequent reallocations of existing arrays
	// will gain tracking if that occurs.  The goal is to filter out startup constructors which run before Main, which introduce a
	// ton of noise into slack reports.  Especially the roughly 30,000 static FString constructors in the code base, each with a
	// unique call stack, and all having a little bit of slack due to malloc bucket size rounding.
	ArraySlackTrackInit();
#endif

	FBootProfiling::InitCounters();

	const int32 rv1 = PreInitPreStartupScreen(CmdLine);
	if (rv1 != 0)
	{
		PreInitContext.Cleanup();
		return rv1;
	}

	const int32 rv2 = PreInitPostStartupScreen(CmdLine);
	if (rv2 != 0)
	{
		PreInitContext.Cleanup();
		return rv2;
	}

	return 0;
}

bool FEngineLoop::LoadCoreModules()
{
	// Always attempt to load CoreUObject. It requires additional pre-init which is called from its module's StartupModule method.
#if WITH_COREUOBJECT
	// Always register the UObjects callbacks from the module manager for dynamic loading/unloading.
	RegisterModularObjectsProcessing();
	return FModuleManager::Get().LoadModule(TEXT("CoreUObject")) != nullptr;
#else
	return true;
#endif
}


void FEngineLoop::CleanupPreInitContext()
{
	PreInitContext.Cleanup();
}

void FEngineLoop::LoadPreInitModules()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Loading PreInit Modules"), STAT_PreInitModules, STATGROUP_LoadTime);

	// GGetMapNameDelegate is initialized here
#if WITH_ENGINE
	FModuleManager::Get().LoadModule(TEXT("Engine"));

	FModuleManager::Get().LoadModule(TEXT("Renderer"));

	FModuleManager::Get().LoadModule(TEXT("AnimGraphRuntime"));

	FPlatformApplicationMisc::LoadPreInitModules();

#if !UE_SERVER
	if (!IsRunningDedicatedServer() )
	{
		if (!GUsingNullRHI)
		{
			// This needs to be loaded before InitializeShaderTypes is called
			FModuleManager::Get().LoadModuleChecked<ISlateRHIRendererModule>("SlateRHIRenderer");
		}
	}
#endif

	FModuleManager::Get().LoadModule(TEXT("Landscape"));

	FModuleManager::Get().LoadModule(TEXT("RHICore"));

	// Initialize ShaderCore before loading or compiling any shaders,
	// But after Renderer and any other modules which implement shader types.
	FModuleManager::Get().LoadModule(TEXT("RenderCore"));

#if WITH_EDITORONLY_DATA
	// Load the texture compressor module before any textures load. They may
	// compress asynchronously and that can lead to a race condition.
	FModuleManager::Get().LoadModule(TEXT("TextureCompressor"));
#endif

	if (!FPlatformProperties::RequiresCookedData())
	{
		FModuleManager::Get().LoadModule(TEXT("Virtualization"));
	}
#endif // WITH_ENGINE

#if WITH_EDITOR
	// Load audio editor module before engine class CDOs are loaded
	FModuleManager::Get().LoadModule(TEXT("AudioEditor"));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) // todo: revist
	FModuleManager::Get().LoadModule(TEXT("AnimationModifiers"));
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif // WITH_EDITOR
}



#if WITH_ENGINE

bool FEngineLoop::LoadStartupCoreModules()
{
	FScopedSlowTask SlowTask(100);

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Loading Startup Modules"), STAT_StartupModules, STATGROUP_LoadTime);

	bool bSuccess = true;

	// Load all Runtime modules
	SlowTask.EnterProgressFrame(10);
	{
		FModuleManager::Get().LoadModule(TEXT("Core"));
		FModuleManager::Get().LoadModule(TEXT("Networking"));
	}

	// Enable the live coding module if it's a developer build
#if WITH_LIVE_CODING
	FModuleManager::Get().LoadModule("LiveCoding");
#endif

	SlowTask.EnterProgressFrame(10);
	FPlatformApplicationMisc::LoadStartupModules();

	// initialize messaging
	SlowTask.EnterProgressFrame(10);
	if (FPlatformProcess::SupportsMultithreading())
	{
		FModuleManager::LoadModuleChecked<IMessagingModule>("Messaging");
	}

	// Init Scene Reconstruction support
#if !UE_SERVER
	if (!IsRunningDedicatedServer())
	{
		FModuleManager::LoadModuleChecked<IMRMeshModule>("MRMesh");
	}
#endif

	SlowTask.EnterProgressFrame(10);
#if WITH_EDITOR
	FModuleManager::Get().LoadModuleChecked("UnrealEd");
	FModuleManager::Get().LoadModuleChecked("LandscapeEditorUtilities");
	FModuleManager::Get().LoadModuleChecked("SubobjectDataInterface");
#endif //WITH_EDITOR

	// Load UI modules
	SlowTask.EnterProgressFrame(10);
	if ( !IsRunningDedicatedServer() )
	{
		FModuleManager::Get().LoadModule("SlateCore");
		FModuleManager::Get().LoadModule("Slate");

#if !UE_BUILD_SHIPPING
		// Need to load up the SlateReflector module to initialize the WidgetSnapshotService
		FModuleManager::Get().LoadModule("SlateReflector");
#endif // !UE_BUILD_SHIPPING
	}

#if WITH_EDITOR
	FModuleManager::Get().LoadModule("EditorStyle");
	// In dedicated server builds with the editor, we need to load UMG/UMGEditor for compiling blueprints.
	// UMG must be loaded for runtime and cooking.
	FModuleManager::Get().LoadModule("UMG");
	// ScriptableEditorWidgets was refactored out of UMG, load it now so that we don't break existing
	// projets that are not listing it in their uproject or uplugin:
	FModuleManager::Get().LoadModule("ScriptableEditorWidgets");
#else
	if ( !IsRunningDedicatedServer() )
	{
		// UMG must be loaded for runtime and cooking.
		FModuleManager::Get().LoadModule("UMG");
	}
#endif //WITH_EDITOR

	// Load all Development modules
	SlowTask.EnterProgressFrame(20);
	if (!IsRunningDedicatedServer())
	{
#if WITH_UNREAL_DEVELOPER_TOOLS
		FModuleManager::Get().LoadModule("MessageLog");
#endif	// WITH_UNREAL_DEVELOPER_TOOLS
#if WITH_EDITOR
		FModuleManager::Get().LoadModule("CollisionAnalyzer");
#endif	// WITH_EDITOR
	}

#if WITH_UNREAL_DEVELOPER_TOOLS
		FModuleManager::Get().LoadModule("FunctionalTesting");
#endif	//WITH_UNREAL_DEVELOPER_TOOLS

	SlowTask.EnterProgressFrame(30);
#if (WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)) // todo: revisit
	// HACK: load BT editor as early as possible for statically initialized assets (non cooked BT assets needs it)
	// cooking needs this module too
	FModuleManager::Get().LoadModule(TEXT("BehaviorTreeEditor"));

	// Ability tasks are based on GameplayTasks, so we need to make sure that module is loaded as well
	FModuleManager::Get().LoadModule(TEXT("GameplayTasksEditor"));
#endif

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
	AudioEditorModule->RegisterAssetActions();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR

#if (WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)) // todo: revisit
	// Load the StringTableEditor module to register its asset actions
	FModuleManager::Get().LoadModule("StringTableEditor");

	if( !IsRunningDedicatedServer() )
	{
		// VREditor needs to be loaded in non-server editor builds early, so engine content Blueprints can be loaded during DDC generation
		FModuleManager::Get().LoadModule(TEXT("VREditor"));
	}

	if (IsRunningCommandlet())
	{
		FModuleManager::Get().LoadModule(TEXT("Blutility"));
	}

#endif //(WITH_EDITOR && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

#if WITH_EDITOR
	if (!IsRunningDedicatedServer())
	{
		FModuleManager::Get().LoadModule(TEXT("MaterialEditor"));
	}
#endif // WITH_EDITOR

#if WITH_ENGINE
	// Load runtime client modules (which are also needed at cook-time)
	if( !IsRunningDedicatedServer() )
	{
		FModuleManager::Get().LoadModule(TEXT("Overlay"));
	}

	FModuleManager::Get().LoadModule(TEXT("MediaAssets"));
#endif

	FModuleManager::Get().LoadModule(TEXT("ClothingSystemRuntimeNv"));
#if WITH_EDITOR
	FModuleManager::Get().LoadModule(TEXT("ClothingSystemEditor"));
	FModuleManager::Get().LoadModule(TEXT("AnimationDataController"));

	// Required during serialization of AnimSequence which could happen from async loading thread.
	// See UAnimSequence::UpdateFrameRate().
	FModuleManager::Get().LoadModule(TEXT("TimeManagement"));

	// Required during construction of UAnimBlueprint which could happen from async loading thread.
	// See UAnimBlueprint::UAnimBlueprint().
	FModuleManager::Get().LoadModule(TEXT("AnimGraph"));

	FModuleManager::Get().LoadModule(TEXT("WorldBookmark"));

	FModuleManager::Get().LoadModule(TEXT("WorldPartitionEditor"));
#endif

	FModuleManager::Get().LoadModule(TEXT("PacketHandler"));
	FModuleManager::Get().LoadModule(TEXT("NetworkReplayStreaming"));

	FModuleManager::Get().LoadModule(TEXT("MassEntity"));

	return bSuccess;
}


bool FEngineLoop::LoadStartupModules()
{
	FScopedSlowTask SlowTask(3);
	LLM_SCOPE_BYNAME(TEXT("Modules"));
	SlowTask.EnterProgressFrame(1);
	// Load any modules that want to be loaded before default modules are loaded up.
	if (!IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PreDefault) || !IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreDefault))
	{
		return false;
	}

	SlowTask.EnterProgressFrame(1);
	// Load modules that are configured to load in the default phase
	if (!IProjectManager::Get().LoadModulesForProject(ELoadingPhase::Default) || !IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::Default))
	{
		return false;
	}

	SlowTask.EnterProgressFrame(1);
	// Load any modules that want to be loaded after default modules are loaded up.
	if (!IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PostDefault) || !IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostDefault))
	{
		return false;
	}

	return true;
}


void FEngineLoop::InitTime()
{
	// Init variables used for benchmarking and ticking.
	FApp::SetCurrentTime(FPlatformTime::Seconds());
	MaxFrameCounter				= 0;
	MaxTickTime					= 0;
	TotalTickTime				= 0;
	LastFrameCycles				= FPlatformTime::Cycles();

	float FloatMaxTickTime		= 0;
#if (!UE_BUILD_SHIPPING || ENABLE_PGO_PROFILE)
	FParse::Value(FCommandLine::Get(),TEXT("SECONDS="),FloatMaxTickTime);
	MaxTickTime					= FloatMaxTickTime;

	// look of a version of seconds that only is applied if FApp::IsBenchmarking() is set. This makes it easier on
	// say, iOS, where we have a toggle setting to enable benchmarking, but don't want to have to make user
	// also disable the seconds setting as well. -seconds= will exit the app after time even if benchmarking
	// is not enabled
	// NOTE: This will override -seconds= if it's specified
	if (FApp::IsBenchmarking())
	{
		if (FParse::Value(FCommandLine::Get(),TEXT("BENCHMARKSECONDS="),FloatMaxTickTime) && FloatMaxTickTime)
		{
			MaxTickTime			= FloatMaxTickTime;
		}
	}

	// Use -FPS=X to override fixed tick rate if e.g. -BENCHMARK is used.
	float FixedFPS = 0;
	FParse::Value(FCommandLine::Get(),TEXT("FPS="),FixedFPS);
	if( FixedFPS > 0 )
	{
		FApp::SetFixedDeltaTime(1 / FixedFPS);
	}

#endif // !UE_BUILD_SHIPPING

	// convert FloatMaxTickTime into number of frames (using 1 / FApp::GetFixedDeltaTime() to convert fps to seconds )
	MaxFrameCounter = FMath::TruncToInt(MaxTickTime / FApp::GetFixedDeltaTime());
}


//called via FCoreDelegates::StarvedGameLoop
void GameLoopIsStarved()
{
	FlushRenderingCommands();
}


int32 FEngineLoop::Init()
{
	ON_SCOPE_EXIT
	{ 
#if TRACK_DISK_UTILIZATION
		CSV_METADATA(TEXT("BootMBLoaded"), *FString::FromInt(GDiskUtilizationTracker.GetLongTermStats().TotalBytesRead / (1024 * 1024)));
#endif
		FBootProfiling::OnInitComplete();
	};
	LLM_SCOPE(ELLMTag::EngineInitMemory);
	SCOPED_BOOT_TIMING("FEngineLoop::Init");

	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FEngineLoop::Init" ), STAT_FEngineLoop_Init, STATGROUP_LoadTime );

	FScopedSlowTask SlowTask(100);
	SlowTask.EnterProgressFrame(10);

	FEmbeddedCommunication::ForceTick(10);

	// Figure out which UEngine variant to use.
	UClass* EngineClass = nullptr;
	if( !GIsEditor )
	{
		SCOPED_BOOT_TIMING("Create GEngine");
		// We're the game.
		FString GameEngineClassName;
		GConfig->GetString(TEXT("/Script/Engine.Engine"), TEXT("GameEngine"), GameEngineClassName, GEngineIni);
		EngineClass = StaticLoadClass( UGameEngine::StaticClass(), nullptr, *GameEngineClassName);
		if (EngineClass == nullptr)
		{
			UE_LOGF(LogInit, Fatal, "Failed to load Game Engine class '%ls'.", *GameEngineClassName);
		}
		GEngine = NewObject<UEngine>(GetTransientPackage(), EngineClass);
	}
	else
	{
#if WITH_EDITOR
		// We're UnrealEd.
		FString UnrealEdEngineClassName;
		GConfig->GetString(TEXT("/Script/Engine.Engine"), TEXT("UnrealEdEngine"), UnrealEdEngineClassName, GEngineIni);
		EngineClass = StaticLoadClass(UUnrealEdEngine::StaticClass(), nullptr, *UnrealEdEngineClassName);
		if (EngineClass == nullptr)
		{
			UE_LOGF(LogInit, Fatal, "Failed to load UnrealEd Engine class '%ls'.", *UnrealEdEngineClassName);
		}
		GEngine = GEditor = GUnrealEd = NewObject<UUnrealEdEngine>(GetTransientPackage(), EngineClass);
#else
		check(0);
#endif
	}

	FEmbeddedCommunication::ForceTick(11);

	check( GEngine );

	if (IsMoviePlayerEnabled())
	{
		GetMoviePlayer()->PassLoadingScreenWindowBackToGame();
	}
    
    if (FPreLoadScreenManager::Get())
    {
        FPreLoadScreenManager::Get()->PassPreLoadScreenWindowBackToGame();
    }

	{
		SCOPED_BOOT_TIMING("GEngine->ParseCommandline()");
		GEngine->ParseCommandline();
	}

	FEmbeddedCommunication::ForceTick(12);

	{
		SCOPED_BOOT_TIMING("InitTime");
		InitTime();
	}

	SlowTask.EnterProgressFrame(60);

	{
		SCOPED_BOOT_TIMING("GEngine->Init");
		GEngine->Init(this);
	}

	// Call init callbacks
	{
		SCOPED_BOOT_TIMING("FCoreDelegates::OnPostEngineInit");
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FCoreDelegates::OnPostEngineInit.Broadcast();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FCoreDelegates::GetOnPostEngineInit().Broadcast();
	}

	SlowTask.EnterProgressFrame(30);

	// initialize engine instance discovery
	if (FPlatformProcess::SupportsMultithreading())
	{
		SCOPED_BOOT_TIMING("SessionService etc");
		if (!IsRunningCommandlet())
		{
			SessionService = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices").GetSessionService();
			
			if (SessionService.IsValid())
			{
				SessionService->Start();
			}
		}

		EngineService = new FEngineService();
		TraceService = new FTraceService();
	}

	{
		SCOPED_BOOT_TIMING("IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PostEngineInit)");
		// Load all the post-engine init modules
		if (!IProjectManager::Get().LoadModulesForProject(ELoadingPhase::PostEngineInit) || !IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostEngineInit))
		{
			RequestEngineExit(TEXT("One or more modules failed PostEngineInit"));
			return 1;
		}
	}

	// Call module loading phases completion callbacks
	SetEngineStartupModuleLoadingComplete();

	// Disable the backlog only after module loading is complete because that gives modules
	// a chance to consume the backlog when constructing output devices.
	GLog->EnableBacklog(false);

	{
		SCOPED_BOOT_TIMING("GEngine->Start()");
		GEngine->Start();
	}

	FEmbeddedCommunication::ForceTick(13);

    if (FPreLoadScreenManager::Get() && FPreLoadScreenManager::Get()->HasActivePreLoadScreenType(EPreLoadScreenTypes::EngineLoadingScreen))
    {
		SCOPED_BOOT_TIMING("WaitForEngineLoadingScreenToFinish");
		FPreLoadScreenManager::Get()->SetEngineLoadingComplete(true);
        FPreLoadScreenManager::Get()->WaitForEngineLoadingScreenToFinish();
    }
    else if (IsMoviePlayerEnabled())
    {
		SCOPED_BOOT_TIMING("WaitForMovieToFinish");
		GetMoviePlayer()->WaitForMovieToFinish();
    }

	FTraceAuxiliary::EnableCommandlineChannels();

#if !UE_SERVER
	// initialize media framework
	IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

	if (MediaModule != nullptr)
	{
		MediaModule->SetTimeSource(MakeShareable(new FAppMediaTimeSource));
	}
#endif

	FEmbeddedCommunication::ForceTick(14);

	// initialize automation worker
#if WITH_AUTOMATION_WORKER
	FModuleManager::Get().LoadModule("AutomationWorker");
#endif

	// Automation tests can be invoked locally in non-editor builds configuration (e.g. performance profiling in Test configuration)
#if WITH_ENGINE && !UE_BUILD_SHIPPING
	FModuleManager::Get().LoadModule("AutomationController");
	FModuleManager::GetModuleChecked<IAutomationControllerModule>("AutomationController").Init();
#endif

#if WITH_EDITOR
	FModuleManager::Get().LoadModule(TEXT("SequenceRecorder"));
	FModuleManager::Get().LoadModule(TEXT("SequenceRecorderSections"));
#endif

	GIsRunning = true;

	if (!GIsEditor)
	{
		// hide a couple frames worth of rendering
		FViewport::SetGameRenderingEnabled(true, 3);
	}

	FEmbeddedCommunication::ForceTick(15);

	FCoreDelegates::StarvedGameLoop.BindStatic(&GameLoopIsStarved);

	// Ready to measure thread heartbeat
	FThreadHeartBeat::Get().Start();

	FShaderPipelineCache::PauseBatching();
   	{
#if defined(WITH_CODE_GUARD_HANDLER) && WITH_CODE_GUARD_HANDLER
         void CheckImageIntegrity();
        CheckImageIntegrity();
#endif
    }
    
    {
		SCOPED_BOOT_TIMING("FCoreDelegates::OnFEngineLoopInitComplete");
		FCoreDelegates::OnFEngineLoopInitComplete.Broadcast();
	}
	FShaderPipelineCache::ResumeBatching();

#if BUILD_EMBEDDED_APP
	FEmbeddedCommunication::AllowSleep(TEXT("Startup"));
	FEmbeddedCommunication::KeepAwake(TEXT("FirstTicks"), true);
#endif
	
#if UE_EXTERNAL_PROFILING_ENABLED
	FExternalProfiler* ActiveProfiler = FActiveExternalProfilerBase::InitActiveProfiler();
	if (ActiveProfiler)
	{
		ActiveProfiler->Register();
	}
#endif		// UE_EXTERNAL_PROFILING_ENABLED

	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::EndOfEngineInit);

	FThreadHeartBeat::Get().InitSettings();
	FGameThreadHitchHeartBeat::Get().InitSettings();

	// Emit logging. Don't edit! Automation looks for this to detect failures during initialization.
	UE_LOGF(LogInit, Display, "Engine is initialized. Leaving FEngineLoop::Init()");
	return 0;
}


// 5.4.2 local change to avoid modifying public headers
namespace PipelineStateCache
{
	// Waits for any pending tasks to complete.
	extern RHI_API void WaitForAllTasks();

}

void FEngineLoop::Exit()
{
	STAT_ADD_CUSTOMMESSAGE_NAME( STAT_NamedMarker, TEXT( "EngineLoop.Exit" ) );
	SCOPED_NAMED_EVENT_TEXT("FEngineLoop::Exit", FrameColor);
	TRACE_BOOKMARK(TEXT("EngineLoop.Exit"));
	
	// Reset all ticker elements. Registered lambda ticker functions can hold on to resources in their captures,
	// so we want to make sure those are released before shutting down.
	FTSTicker::GetCoreTicker().Reset();

	ClearPendingCleanupObjects();

	GIsRunning	= 0;
	GLogConsole	= nullptr;

	IInstallBundleManager::InstallBundleCompleteDelegate.RemoveAll(this);

	if (FPreLoadScreenManager::Get())
	{
		// If we exit before the preload screen is done, clean it up before shutting down
		if (FPreLoadScreenManager::Get()->HasActivePreLoadScreenType(EPreLoadScreenTypes::EngineLoadingScreen))
		{
			FPreLoadScreenManager::Get()->SetEngineLoadingComplete(true);
			FPreLoadScreenManager::Get()->WaitForEngineLoadingScreenToFinish();
		}

		FPreLoadScreenManager::Destroy();
	}

	// shutdown visual logger and flush all data
#if ENABLE_VISUAL_LOG
	FVisualLogger::Get().TearDown();
#endif

	FAssetCompilingManager::Get().Shutdown();

#if WITH_ENGINE
	// shut down messaging
	delete EngineService;
	EngineService = nullptr;

	delete TraceService;
	TraceService = nullptr;

	if (SessionService.IsValid())
	{
		SessionService->Stop();
		SessionService.Reset();
	}

	if (GDistanceFieldAsyncQueue)
	{
		delete GDistanceFieldAsyncQueue;
		GDistanceFieldAsyncQueue = nullptr;
	}

	if (GCardRepresentationAsyncQueue)
	{
		delete GCardRepresentationAsyncQueue;
		GCardRepresentationAsyncQueue = nullptr;
	}
#endif // WITH_ENGINE

	// Make sure we're not in the middle of loading something after this point as some
	// systems are shut down during PreExit and it can cause issues and crashes if loading continues
	// beyond this point.
	FlushAsyncLoading();

	if ( GEngine != nullptr )
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnEditorPreExit", CoreDelegatesColor);
			FEditorDelegates::OnEditorPreExit.Broadcast();
		}
#endif // WITH_EDITOR

		GEngine->PreExit();
	}

	if (GEngine != nullptr)
	{
		GEngine->ReleaseAudioDeviceManager();
	}


	// Make sure we're not in the middle of loading something.
	{
		FlushAsyncLoading();

		// From now on it's not allowed to request new async loads
		// any new requests done during the scope of the flush should have been flushed as well, now prevent any new requests
		SetAsyncLoadingAllowed(false);
	}

	// Block till all outstanding resource streaming requests are fulfilled.
	if (!IStreamingManager::HasShutdown())
	{
		UTexture2D::CancelPendingTextureStreaming();
		if (FStreamingManagerCollection* StreamingManager = IStreamingManager::Get_Concurrent())
		{
			StreamingManager->BlockTillAllRequestsFinished();
		}
	}
	FAudioDeviceManager::Shutdown();

	// close all windows
	FSlateApplication::Shutdown();

#if !UE_SERVER
	if ( FEngineFontServices::IsInitialized() )
	{
		FEngineFontServices::Destroy();
	}
#endif



#if WITH_EDITOR
	// These module must be shut down first because other modules may try to access them during shutdown.
	// Accessing these modules at shutdown causes instability since the object system will have been shut down and these modules uses uobjects internally.
	FModuleManager::Get().UnloadModule("AssetTools", true);

#endif // WITH_EDITOR
	FModuleManager::Get().UnloadModule("WorldBrowser", true);


#if WITH_ENGINE	
	// Reset any in progress PSO compile requests, reduces pipelinestatecache task wait time.
	ClearMaterialPSORequests();
#endif

	// Wait for any pending tasks to complete.
	PipelineStateCache::WaitForAllTasks();

	AppPreExit();

	TermGamePhys();

#if WITH_EDITOR
	IBulkDataRegistry::Shutdown();
#endif

#if WITH_COREUOBJECT
	// PackageResourceManager depends on AssetRegistry, so must be shutdown before we unload the AssetRegistry module
	IPackageResourceManager::Shutdown();
#endif
	FModuleManager::Get().UnloadModule("AssetRegistry", true);

	// Stop the rendering thread.
	ShutdownRenderingThread();

	// Disable the PSO cache
	FShaderPipelineCache::Shutdown();

	// Free the global shader map, needs to happen before FShaderCodeLibrary::Shutdown to avoid warnings
	// about leaking RHI references.
	ShutdownGlobalShaderMap();

	// Close shader code map, if any
	FShaderCodeLibrary::Shutdown();

	// Stop IoDispatcher after FShaderCodeLibrary, as it holds on to file requests
#if WITH_ENGINE
	UE::DerivedData::IoStore::TearDownIoDispatcher();
#endif
#if USE_IO_DISPATCHER
	FIoDispatcher::Shutdown();
	FModuleManager::Get().UnloadModule("IoStoreOnDemandCore", true);
#if UE_WITH_IOSTOREONDEMAND
	FModuleManager::Get().UnloadModule("IoStoreOnDemand", true);
#endif
#endif

#if WITH_EDITOR
	// Make sure we shut this down before the modules are torn down, we can clean this up if/when the
	// virtualization module is moved to be a plugin
	UE::Virtualization::Shutdown();
#endif //WITH_EDITOR

#if WITH_ENGINE
	// Save the hot reload state
	IHotReloadInterface* HotReload = IHotReloadInterface::GetPtr();
	if(HotReload != nullptr)
	{
		HotReload->SaveConfig();
	}
#endif

	// Unload all modules.  Note that this doesn't actually unload the module DLLs (that happens at
	// process exit by the OS), but it does call ShutdownModule() on all loaded modules in the reverse
	// order they were loaded in, so that systems can unregister and perform general clean up.
	{
		SCOPED_BOOT_TIMING("UnloadModulesAtShutdown");
		FModuleManager::Get().UnloadModulesAtShutdown();
	}

	IStreamingManager::Shutdown();

	DestroyMoviePlayer();

	// Move earlier?
#if STATS
	FThreadStats::StopThread();
#endif

	// Clean up the thread pool
	// GThreadPool might be a wrapper around GLargeThreadPool so we have to destroy it first
	if (GThreadPool != nullptr)
	{
		GThreadPool->Destroy();
	}

#if WITH_EDITOR
	if (GLargeThreadPool != nullptr)
	{
		GLargeThreadPool->Destroy();
	}
#endif // WITH_EDITOR

	if (GBackgroundPriorityThreadPool != nullptr)
	{
		GBackgroundPriorityThreadPool->Destroy();
	}

	RHIExit();

	FTaskGraphInterface::Shutdown();

	FPlatformMisc::ShutdownTaggedStorage();

#if WITH_EDITOR && PLATFORM_WINDOWS
	FWindowsPlatformPerfCounters::Shutdown();
#endif

#if WITH_ENGINE && FRAMEPRO_ENABLED
	FFrameProProfiler::TearDown();
#endif // FRAMEPRO_ENABLED

	UE::Trace::Exit();
}


void FEngineLoop::ProcessLocalPlayerSlateOperations() const
{
	FSlateApplication& SlateApp = FSlateApplication::Get();

	// For all the game worlds drill down to the player controller for each game viewport and process it's slate operation
	for ( const FWorldContext& Context : GEngine->GetWorldContexts() )
	{
		UWorld* CurWorld = Context.World();
		if ( CurWorld && CurWorld->IsGameWorld() )
		{
			UGameViewportClient* GameViewportClient = CurWorld->GetGameViewport();
			TSharedPtr< SViewport > ViewportWidget = GameViewportClient ? GameViewportClient->GetGameViewportWidget() : nullptr;

			if ( ViewportWidget.IsValid() )
			{
				FWidgetPath PathToWidget;
				SlateApp.GeneratePathToWidgetUnchecked(ViewportWidget.ToSharedRef(), PathToWidget);

				if (PathToWidget.IsValid())
				{
					for (FConstPlayerControllerIterator Iterator = CurWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
					{
						APlayerController* PlayerController = Iterator->Get();
						if (PlayerController)
						{
							if (ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PlayerController->Player))
							{
								TOptional<int32> UserIndex = SlateApp.GetUserIndexForController(LocalPlayer->GetControllerId());
								if (UserIndex.IsSet())
								{
									FReply& TheReply = LocalPlayer->GetSlateOperations();
									SlateApp.ProcessExternalReply(PathToWidget, TheReply, UserIndex.GetValue());

									TheReply = FReply::Unhandled();
								}
							}
						}
					}
				}
			}
		}
	}
}

#if WITH_ENGINE
void OnStartupContentMounted(FInstallBundleRequestResultInfo Result, bool bDumpEarlyConfigReads, bool bDumpEarlyPakFileReads, bool bReloadConfig, bool bForceQuitAfterEarlyReads)
{
	if (Result.bIsStartup && Result.Result == EInstallBundleResult::OK)
	{
		DumpEarlyReads(bDumpEarlyConfigReads, bDumpEarlyPakFileReads, bForceQuitAfterEarlyReads);
		HandleConfigReload(bReloadConfig);

		IInstallBundleManager::InstallBundleCompleteDelegate.RemoveAll(&GEngineLoop);
	}
}
#endif

void DumpEarlyReads(bool bDumpEarlyConfigReads, bool bDumpEarlyPakFileReads, bool bForceQuitAfterEarlyReads)
{
	if (bDumpEarlyConfigReads)
	{
		UE::ConfigUtilities::DumpRecordedConfigReadsFromIni();
		UE::ConfigUtilities::DeleteRecordedConfigReadsFromIni();
	}

	if (bDumpEarlyPakFileReads)
	{
		DumpRecordedFileReadsFromPaks();
		DeleteRecordedFileReadsFromPaks();
	}

	if (bForceQuitAfterEarlyReads)
	{
		GLog->Flush();
		if (GEngine)
		{
			GEngine->DeferredCommands.Emplace(TEXT("Quit force"));
		}
		else
		{
			FPlatformMisc::RequestExit(true, TEXT("DumpEarlyReads"));
		}
	}
}

void HandleConfigReload(bool bReloadConfig)
{
	if (bReloadConfig)
	{
		UE::ConfigUtilities::ReapplyRecordedCVarSettingsFromIni();
		UE::ConfigUtilities::DeleteRecordedCVarSettingsFromIni();
	}
}

bool FEngineLoop::ShouldUseIdleMode() const
{
	static const auto CVarIdleWhenNotForeground = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("t.IdleWhenNotForeground"));
	bool bIdleMode = false;

	// Yield cpu usage if desired
	if (FApp::IsGame()
		&& FPlatformProperties::SupportsWindowedMode()
		&& CVarIdleWhenNotForeground->GetValueOnGameThread()
		&& !FApp::HasFocus())
	{
		bIdleMode = true;
	}
	
#if BUILD_EMBEDDED_APP
	if (FEmbeddedCommunication::IsAwakeForTicking() == false)
	{
		bIdleMode = true;
	}
#endif

	if (bIdleMode)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (!Context.World()->AreAlwaysLoadedLevelsLoaded())
			{
				bIdleMode = false;
				break;
			}
		}
	}

	return bIdleMode;
}

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST && MALLOC_GT_HOOKS

#include "Containers/StackTracker.h"
static TAutoConsoleVariable<int32> CVarLogGameThreadMallocChurn(
	TEXT("LogGameThreadMallocChurn.Enable"),
	0,
	TEXT("If > 0, then collect sample game thread malloc, realloc and free, periodically print a report of the worst offenders."));

static TAutoConsoleVariable<int32> CVarLogGameThreadMallocChurn_PrintFrequency(
	TEXT("LogGameThreadMallocChurn.PrintFrequency"),
	300,
	TEXT("Number of frames between churn reports."));

static TAutoConsoleVariable<int32> CVarLogGameThreadMallocChurn_Threshhold(
	TEXT("LogGameThreadMallocChurn.Threshhold"),
	10,
	TEXT("Minimum average number of allocs per frame to include in the report."));

static TAutoConsoleVariable<int32> CVarLogGameThreadMallocChurn_SampleFrequency(
	TEXT("LogGameThreadMallocChurn.SampleFrequency"),
	100,
	TEXT("Number of allocs to skip between samples. This is used to prevent churn sampling from slowing the game down too much."));

static TAutoConsoleVariable<int32> CVarLogGameThreadMallocChurn_StackIgnore(
	TEXT("LogGameThreadMallocChurn.StackIgnore"),
	2,
	TEXT("Number of items to discard from the top of a stack frame."));

static TAutoConsoleVariable<int32> CVarLogGameThreadMallocChurn_RemoveAliases(
	TEXT("LogGameThreadMallocChurn.RemoveAliases"),
	1,
	TEXT("If > 0 then remove aliases from the counting process. This essentialy merges addresses that have the same human readable string. It is slower."));

static TAutoConsoleVariable<int32> CVarLogGameThreadMallocChurn_StackLen(
	TEXT("LogGameThreadMallocChurn.StackLen"),
	3,
	TEXT("Maximum number of stack frame items to keep. This improves aggregation because calls that originate from multiple places but end up in the same place will be accounted together."));


extern CORE_API TFunction<void(int32)>* GGameThreadMallocHook;

struct FScopedSampleMallocChurn
{
	static FStackTracker GGameThreadMallocChurnTracker;
	static uint64 DumpFrame;

	bool bEnabled;
	int32 CountDown;
	TFunction<void(int32)> Hook;

	FScopedSampleMallocChurn()
		: bEnabled(CVarLogGameThreadMallocChurn.GetValueOnGameThread() > 0)
		, CountDown(CVarLogGameThreadMallocChurn_SampleFrequency.GetValueOnGameThread())
		, Hook(
		[this](int32 Index)
	{
		if (--CountDown <= 0)
		{
			CountDown = CVarLogGameThreadMallocChurn_SampleFrequency.GetValueOnGameThread();
			CollectSample();
		}
	}
	)
	{
		if (bEnabled)
		{
			check(IsInGameThread());
			check(!GGameThreadMallocHook);
			if (!DumpFrame)
			{
				DumpFrame = GFrameCounter + CVarLogGameThreadMallocChurn_PrintFrequency.GetValueOnGameThread();
				GGameThreadMallocChurnTracker.ResetTracking();
			}
			GGameThreadMallocChurnTracker.ToggleTracking(true, true);
			GGameThreadMallocHook = &Hook;
		}
		else
		{
			check(IsInGameThread());
			GGameThreadMallocChurnTracker.ToggleTracking(false, true);
			if (DumpFrame)
			{
				DumpFrame = 0;
				GGameThreadMallocChurnTracker.ResetTracking();
			}
		}
	}
	~FScopedSampleMallocChurn()
	{
		if (bEnabled)
		{
			check(IsInGameThread());
			check(GGameThreadMallocHook == &Hook);
			GGameThreadMallocHook = nullptr;
			GGameThreadMallocChurnTracker.ToggleTracking(false, true);
			check(DumpFrame);
			if (GFrameCounter > DumpFrame)
			{
				PrintResultsAndReset();
			}
		}
	}

	void CollectSample()
	{
		check(IsInGameThread());
		GGameThreadMallocChurnTracker.CaptureStackTrace(CVarLogGameThreadMallocChurn_StackIgnore.GetValueOnGameThread(), nullptr, CVarLogGameThreadMallocChurn_StackLen.GetValueOnGameThread(), CVarLogGameThreadMallocChurn_RemoveAliases.GetValueOnGameThread() > 0);
	}
	void PrintResultsAndReset()
	{
		DumpFrame = GFrameCounter + CVarLogGameThreadMallocChurn_PrintFrequency.GetValueOnGameThread();
		FOutputDeviceRedirector* Log = FOutputDeviceRedirector::Get();
		float SampleAndFrameCorrection = float(CVarLogGameThreadMallocChurn_SampleFrequency.GetValueOnGameThread()) / float(CVarLogGameThreadMallocChurn_PrintFrequency.GetValueOnGameThread());
		GGameThreadMallocChurnTracker.DumpStackTraces(CVarLogGameThreadMallocChurn_Threshhold.GetValueOnGameThread(), *Log, SampleAndFrameCorrection);
		GGameThreadMallocChurnTracker.ResetTracking();
	}
};
FStackTracker FScopedSampleMallocChurn::GGameThreadMallocChurnTracker;
uint64 FScopedSampleMallocChurn::DumpFrame = 0;

#endif

#if WITH_RHI_BREADCRUMBS
TOptional<FRHIBreadcrumbEventManual> GRHIFrameBreadcrumb;
#endif

static inline void BeginFrameRenderThread(FRHICommandListImmediate& RHICmdList, uint64 CurrentFrameCounter)
{
	if ( !FApp::CanEverRender() )
	{
		GFrameNumberRenderThread++;
		return;
	}

	TRACE_BEGIN_FRAME(TraceFrameType_Rendering);
	GFrameNumberRenderThread++;

#if WITH_RHI_BREADCRUMBS
#if CSV_PROFILER
	if (FCsvProfiler::Get()->IsCapturing_Renderthread())
	{
		GRHIFrameBreadcrumb.Emplace(RHICmdList, RHI_BREADCRUMB_DESC_FORWARD_VALUES(TEXT("Frame"), TEXT("Frame %d (CsvFrame %d)"), RHI_GPU_STAT_ARGS_NONE)(
			  RHI_BREADCRUMB_FIELD("Frame Number", CurrentFrameCounter)
			, RHI_BREADCRUMB_FIELD("Csv Frame Number", FCsvProfiler::Get()->GetCaptureFrameNumberRT())
		));
	}
	else
#endif // CSV_PROFILER
	{
		GRHIFrameBreadcrumb.Emplace(RHICmdList, RHI_BREADCRUMB_DESC_FORWARD_VALUES(TEXT("Frame"), TEXT("Frame %d"), RHI_GPU_STAT_ARGS_NONE)(
			RHI_BREADCRUMB_FIELD("Frame Number", CurrentFrameCounter)
		));
	}
#endif // WITH_RHI_BREADCRUMBS

	{
		SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnBeginFrameRT", CoreDelegatesColor);
		FCoreDelegates::OnBeginFrameRT.Broadcast();
	}

	RHICmdList.EnqueueLambda([CurrentFrameCounter](FRHICommandListImmediate& InRHICmdList)
	{
		UEngine::SetRenderSubmitLatencyMarkerStart(CurrentFrameCounter);
	});

#if CSV_PROFILER_STATS
	FCsvProfiler::BeginExclusiveStat("RenderThreadOther");
#endif

	GVisualizeTexture.BeginFrameRenderThread();

	// Waits after this point but before BeginRenderingViewFamilies are not on the RT critical path (since we're waiting for the GT)
	UE::Stats::FThreadIdleStats::EndCriticalPath();
}


static inline void EndFrameRenderThread(FRHICommandListImmediate& RHICmdList, uint64 CurrentFrameCounter)
{
	if ( !FApp::CanEverRender() )
	{
		RHICmdList.EndFrame();
		return;
	}

#if CSV_PROFILER_STATS
	FCsvProfiler::EndExclusiveStat("RenderThreadOther");
#endif

	RHICmdList.EnqueueLambda([CurrentFrameCounter](FRHICommandListImmediate& InRHICmdList)
	{
		UEngine::SetRenderSubmitLatencyMarkerEnd(CurrentFrameCounter);
	});

	{
		SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnEndFrameRT", CoreDelegatesColor);
		FCoreDelegates::OnEndFrameRT.Broadcast();
	}

	GVisualizeTexture.EndFrameRenderThread();

#if WITH_RHI_BREADCRUMBS
	GRHIFrameBreadcrumb->End(RHICmdList);
	GRHIFrameBreadcrumb.Reset();
#endif

	RHICmdList.EndFrame();

	TRACE_END_FRAME(TraceFrameType_Rendering);
}

#if BUILD_EMBEDDED_APP
#include "Misc/EmbeddedCommunication.h"
#endif

void FEngineLoop::Tick()
{
	SCOPED_NAMED_EVENT_TEXT("FEngineLoop::Tick", FrameColor);
	SCOPE_STALL_COUNTER(FEngineLoop::Tick, 2.0);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST && MALLOC_GT_HOOKS
	FScopedSampleMallocChurn ChurnTracker;
#endif
	// let the low level mem tracker pump once a frame to update states
	LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());

	LLM_SCOPE(ELLMTag::EngineMisc);

	BeginExitIfRequested();
#if !UE_BUILD_SHIPPING
	if (GScopedTestExit.IsValid() && GScopedTestExit->RequestExit())
	{
		UE_LOGF(LogExit, Display, "**** TestExit: %ls ****", *GScopedTestExit->RequestExitPhrase());
		FPlatformMisc::RequestExit(true, TEXT("FEngineLoop::Tick.GScopedTestExit"));
	}
#endif

	{
		SCOPED_NAMED_EVENT(FEngineLoop_HeartBeat, SystemColor);
		// Send a heartbeat for the diagnostics thread
		FThreadHeartBeat::Get().HeartBeat();

		FGameThreadHitchHeartBeat::Get().FrameStart();
	}

	// Must be called on the game thread outside of any frame breadcrumbs.
	LatchRenderThreadConfiguration();

	// Make sure something is ticking the rendering tickables in -onethread mode to avoid leaks/bugs.
	if (!GUseThreadedRendering)
	{
		SCOPED_NAMED_EVENT(Tick_RenderingTickables, SystemColor);
		TickRenderingTickables(FRHICommandListImmediate::Get());
	}

	// Ensure we aren't starting a frame while loading or playing a loading movie
	FMoviePlayerProxy::BlockingForceFinished();
	ensure(!IsMoviePlayerEnabled() || (GetMoviePlayer()->IsLoadingFinished() && !GetMoviePlayer()->IsMovieCurrentlyPlaying()));

    // Frame profiling kickoff
#if UE_EXTERNAL_PROFILING_ENABLED
	FExternalProfiler* ActiveProfiler = FActiveExternalProfilerBase::GetActiveProfiler();
	if (ActiveProfiler)
	{
		ActiveProfiler->FrameSync();
	}
#endif // UE_EXTERNAL_PROFILING_ENABLED

	FPlatformMisc::BeginNamedEventFrame();

	uint64 CurrentFrameCounter = GFrameCounter;

#if ENABLE_NAMED_EVENTS
	bool bTraceCpuChannelEnabled = false;
#if CPUPROFILERTRACE_ENABLED
	bTraceCpuChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel);
	static FAutoNamedEventsToggler TraceNamedEventsToggler;
	TraceNamedEventsToggler.Update(bTraceCpuChannelEnabled && UE::Trace::IsTracing());
#endif // CPUPROFILERTRACE_ENABLED

	// Generate the Frame event string
	TCHAR IndexedFrameString[32] = { 0 };
	const TCHAR* FrameStringPrefix = bTraceCpuChannelEnabled ? TEXT(" ") : TEXT("Frame ");
#if CSV_PROFILER
	if (FCsvProfiler::Get()->IsCapturing())
	{
		FCString::Snprintf(IndexedFrameString, 32, TEXT("Csv%s%d"), FrameStringPrefix, FCsvProfiler::Get()->GetCaptureFrameNumber());
	}
	else
#endif // CSV_PROFILER
	{
		FCString::Snprintf(IndexedFrameString, 32, TEXT("%s%d"), FrameStringPrefix, CurrentFrameCounter);
	}
	const TCHAR* FrameString = IndexedFrameString;
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, Frame, CpuChannel) 
		<< Frame.Name(FrameString);
#endif // CPUPROFILERTRACE_ENABLED

#if PLATFORM_LIMIT_PROFILER_UNIQUE_NAMED_EVENTS
	FrameString = TEXT("FEngineLoop");
#endif
	SCOPED_NAMED_EVENT_TCHAR_CONDITIONAL(FrameString, FrameColor, !bTraceCpuChannelEnabled);
#endif // ENABLE_NAMED_EVENTS

	// execute callbacks for cvar changes
	{
		SCOPED_NAMED_EVENT(FEngineLoop_CallAllConsoleVariableSinks, SystemColor);
		IConsoleManager::Get().CallAllConsoleVariableSinks();
	}

	TRACE_BEGIN_FRAME(TraceFrameType_Game);

	{
		SCOPE_CYCLE_COUNTER(STAT_FrameTime);

#if WITH_ENGINE && CSV_PROFILER
		UpdateCoreCsvStats_BeginFrame();
#endif

		{
			SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnBeginFrame", CoreDelegatesColor);
			FCoreDelegates::OnBeginFrame.Broadcast();
		}

		// flush debug output which has been buffered by other threads
		{
			SCOPED_NAMED_EVENT(FEngineLoop_FlushThreadedLogs, SystemColor);
			GLog->FlushThreadedLogs(EOutputDeviceRedirectorFlushOptions::Async);
		}

		// exit if frame limit is reached in benchmark mode, or if time limit is reached
		if ((FApp::IsBenchmarking() && MaxFrameCounter && (GFrameCounter > MaxFrameCounter)) ||
			(MaxTickTime && (TotalTickTime > MaxTickTime)))
		{
			FPlatformMisc::RequestExit(false, TEXT("FEngineLoop::Tick.Benchmarking"));
		}

		// set FApp::CurrentTime, FApp::DeltaTime and potentially wait to enforce max tick rate
		{
			SCOPED_NAMED_EVENT(FEngineLoop_UpdateTimeAndHandleMaxTickRate, SystemColor);
			GEngine->UpdateTimeAndHandleMaxTickRate();
			GEngine->SetSimulationLatencyMarkerStart(CurrentFrameCounter);
#if WITH_ENGINE && CSV_PROFILER
			FCsvProfiler::Get()->UpdateFrameTime();
#endif
		}

		// beginning of RHI frame
		{
			SCOPED_NAMED_EVENT(Render_BeginFrame, SystemColor);

#if WITH_RHI_BREADCRUMBS
			// Uses _NOMARKER to avoid intersecting "BeginFrame" scope with the "Frame" breadcrumb from BeginFrameRenderThread().
			ENQUEUE_RENDER_COMMAND_NOMARKER(BeginFrame)([CurrentFrameCounter](FRHICommandListImmediate& RHICmdList)
			{
				BeginFrameRenderThread(RHICmdList, CurrentFrameCounter);
			});
#else // WITH_RHI_BREADCRUMBS
			ENQUEUE_RENDER_COMMAND(BeginFrame)([CurrentFrameCounter](FRHICommandListImmediate& RHICmdList)
			{
				BeginFrameRenderThread(RHICmdList, CurrentFrameCounter);
			});
#endif // WITH_RHI_BREADCRUMBS

			for (FSceneInterface* Scene : GetRendererModule().GetAllocatedScenes())
			{
				ENQUEUE_RENDER_COMMAND(FScene_StartFrame)([Scene](FRHICommandListImmediate& RHICmdList)
				{
					Scene->StartFrame();
				});
			}

			UE::RenderCommandPipe::StartRecording();

#if !UE_SERVER && WITH_ENGINE
			if (!GIsEditor && GEngine->GameViewport && GEngine->GameViewport->GetWorld() && GEngine->GameViewport->GetWorld()->IsCameraMoveable())
			{
				// When not in editor, we emit dynamic resolution's begin frame right after RHI's.
				GEngine->EmitDynamicResolutionEvent(EDynamicResolutionStateEvent::BeginFrame);
			}
#endif
		}

		// tick performance monitoring
		{
			SCOPED_NAMED_EVENT(Tick_PerformanceMonitoring, SystemColor);
			GEngine->TickPerformanceMonitoring( FApp::GetDeltaTime() );

			extern COREUOBJECT_API void ResetAsyncLoadingStats();
			ResetAsyncLoadingStats();
		}

#if UPDATE_MALLOC_STATS
		// update memory allocator stats
		{
			SCOPED_NAMED_EVENT(Malloc_UpdateStats, SystemColor);
			UE::Private::GMalloc->UpdateStats();
		}
#endif
	}

	UE::Stats::FStats::AdvanceFrame( false, UE::Stats::FStats::FOnAdvanceRenderingThreadStats::CreateStatic( &AdvanceRenderingThreadStatsGT ) );

	{
		SCOPE_CYCLE_COUNTER( STAT_FrameTime );

		// Calculates average FPS/MS (outside STATS on purpose)
		CalculateFPSTimings();

		// handle some per-frame tasks on the rendering thread
		ENQUEUE_RENDER_COMMAND(ResetDeferredUpdates)(
			[](FRHICommandListImmediate& RHICmdList)
			{
				FDeferredUpdateResource::ResetNeedsUpdate();
			});

		// Don't pump messages if we're running embedded as the outer application
		// will pass us messages instead.
		if (!GUELibraryOverrideSettings.bIsEmbedded)
		{
			GEngine->SetInputSampleLatencyMarker(CurrentFrameCounter);

			SCOPED_NAMED_EVENT(FEngineLoop_PumpMessages, SystemColor);
			FPlatformApplicationMisc::PumpMessages(true);
		}

		bool bIdleMode;
		{
			SCOPED_NAMED_EVENT(FEngineLoop_Idle, SystemColor);

			// Idle mode prevents ticking and rendering completely
			bIdleMode = ShouldUseIdleMode();
			if (bIdleMode)
			{
				// Yield CPU time
				FPlatformProcess::Sleep(.1f);
			}
		}

		// @todo vreditor urgent: Temporary hack to allow world-to-meters to be set before
		// input is polled for motion controller devices each frame.
		extern ENGINE_API float GNewWorldToMetersScale;
		if( GNewWorldToMetersScale != 0.0f  )
		{
#if WITH_ENGINE
			UWorld* WorldToScale = GWorld;

#if WITH_EDITOR
			if( GIsEditor && GEditor->PlayWorld != nullptr && GEditor->bIsSimulatingInEditor )
			{
				WorldToScale = GEditor->PlayWorld;
			}
#endif //WITH_EDITOR

			if( WorldToScale != nullptr )
			{
				if( GNewWorldToMetersScale != WorldToScale->GetWorldSettings()->WorldToMeters )
				{
					WorldToScale->GetWorldSettings()->WorldToMeters = GNewWorldToMetersScale;
				}
			}

			GNewWorldToMetersScale = 0.0f;
		}
#endif //WITH_ENGINE

		// tick active platform files
		FPlatformFileManager::Get().TickActivePlatformFile();

		// Roughly track the time when the input was sampled
		{
			SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnSamplingInput", CoreDelegatesColor);
			FCoreDelegates::OnSamplingInput.Broadcast();
		}

		// process accumulated Slate input
		if (FSlateApplication::IsInitialized() && !bIdleMode)
		{
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Input);
			SCOPE_TIME_GUARD(TEXT("SlateInput"));
			SCOPED_NAMED_EVENT(Tick_SlateInput, SystemColor);
			LLM_SCOPE(ELLMTag::UI);

			FSlateApplication& SlateApp = FSlateApplication::Get();
            {
				TRACE_CPUPROFILER_EVENT_SCOPE(PollGameDeviceState);
                SlateApp.PollGameDeviceState();
            }
			// Gives widgets a chance to process any accumulated input
            {
				TRACE_CPUPROFILER_EVENT_SCOPE(FinishedInputThisFrame);
                SlateApp.FinishedInputThisFrame();
            }
		}
		
		{
			SCOPED_NAMED_EVENT(Tick_Engine, SystemColor);
			// main game engine tick (world, game objects, etc.)
			GEngine->Tick(FApp::GetDeltaTime(), bIdleMode);
		}

		// If a movie that is blocking the game thread has been playing,
		// wait for it to finish before we continue to tick or tick again
		// We do this right after GEngine->Tick() because that is where user code would initiate a load / movie.
		{
            if (FPreLoadScreenManager::Get())
            {
                if (FPreLoadScreenManager::Get()->HasRegisteredPreLoadScreenType(EPreLoadScreenTypes::EngineLoadingScreen))
                {
                    //Wait for any Engine Loading Screen to stop
                    if (FPreLoadScreenManager::Get()->HasActivePreLoadScreenType(EPreLoadScreenTypes::EngineLoadingScreen))
                    {
                        FPreLoadScreenManager::Get()->WaitForEngineLoadingScreenToFinish();
                    }

                    //Switch Game Window Back
                    UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
                    if (GameEngine)
                    {
                        GameEngine->SwitchGameWindowToUseGameViewport();
                    }
                }

#if !UE_SERVER
				// Is it ok to start up the movie player?
				if (IsMoviePlayerEnabled() && !GetMoviePlayer()->IsMovieCurrentlyPlaying())
				{
					// Enable the MoviePlayer now that the preload screen manager is done.
					if (FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer())
					{
						GetMoviePlayer()->Initialize(*Renderer, FPreLoadScreenManager::Get()->GetRenderWindow());
					}
				}
#endif // !UE_SERVER

                //Destroy / Clean Up PreLoadScreenManager as we are now done
                FPreLoadScreenManager::Destroy();
            }
			else if (IsMoviePlayerEnabled())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(WaitForMovieToFinish);
				GetMoviePlayer()->WaitForMovieToFinish(true);
			}
		}

		FAssetCompilingManager::Get().ProcessAsyncTasks(true);

		FMoviePlayerProxy::BlockingForceFinished();
		// Tick the platform and input portion of Slate application, we need to do this before we run things
		// concurrent with networking.
		if (FSlateApplication::IsInitialized() && !bIdleMode)
		{
			{
				SCOPED_NAMED_EVENT(ProcessLocalPlayerSlateOperations, SystemColor);
				check(!IsRunningDedicatedServer());

				// Process slate operations accumulated in the world ticks.
				ProcessLocalPlayerSlateOperations();
			}

			FSlateApplication::Get().Tick(ESlateTickType::PlatformAndInput);
		}

#if WITH_ENGINE
		// process concurrent Slate tasks
		FGraphEventRef ConcurrentTask;
		FEvent* ConcurrentTaskCompleteEvent = nullptr;
		const bool bDoConcurrentSlateTick = GEngine->ShouldDoAsyncEndOfFrameTasks();

		const UGameViewportClient* const GameViewport = GEngine->GameViewport;
		const UWorld* const GameViewportWorld = GameViewport ? GameViewport->GetWorld() : nullptr;
		UDemoNetDriver* const CurrentDemoNetDriver = GameViewportWorld ? GameViewportWorld->GetDemoNetDriver() : nullptr;

		const bool bValidateReplicatedProperties = CurrentDemoNetDriver && CVarDoAsyncEndOfFrameTasksValidateReplicatedProperties.GetValueOnGameThread() != 0;

		if (bDoConcurrentSlateTick)
		{
			const float DeltaSeconds = FApp::GetDeltaTime();

			if (CurrentDemoNetDriver && CurrentDemoNetDriver->ShouldTickFlushAsyncEndOfFrame())
			{
				ConcurrentTaskCompleteEvent = FPlatformProcess::GetSynchEventFromPool();
				check(ConcurrentTaskCompleteEvent);

				ConcurrentTask = TGraphTask<FExecuteConcurrentWithSlateTickTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndDispatchWhenReady(
					[CurrentDemoNetDriver, DeltaSeconds]()
				{
					if (CVarDoAsyncEndOfFrameTasksRandomize.GetValueOnAnyThread(true) > 0)
					{
						FPlatformProcess::Sleep(FMath::RandRange(0.0f, .003f)); // this shakes up the threading to find race conditions
					}

					if (CurrentDemoNetDriver != nullptr)
					{
						CurrentDemoNetDriver->TickFlushAsyncEndOfFrame(DeltaSeconds);
					}
				},
				ConcurrentTaskCompleteEvent);
				check(ConcurrentTask.IsValid());

				// If we're validating, we want to only test the slate tick so wait for the task
				if (bValidateReplicatedProperties)
				{
					CSV_SCOPED_SET_WAIT_STAT(ConcurrentWithSlate);

					TRACE_CPUPROFILER_EVENT_SCOPE(ConcurrentWithSlateTickTasks_Wait);
					check(ConcurrentTaskCompleteEvent);
					ConcurrentTaskCompleteEvent->Wait();
					FPlatformProcess::ReturnSynchEventToPool(ConcurrentTaskCompleteEvent);
					ConcurrentTaskCompleteEvent = nullptr;
					ConcurrentTask = nullptr;
				}
			}
		}

		// Optionally validate that Slate has not modified any replicated properties for client replay recording.
		FDemoSavedPropertyState PreSlateObjectStates;
		if (bValidateReplicatedProperties)
		{
			PreSlateObjectStates = CurrentDemoNetDriver->SavePropertyState();
		}
#endif

		// Tick(Advance) Time for the application and then tick and paint slate application widgets.
		// We split separate this action from the one above to permit running network replication concurrent with slate widget ticking and painting.
		if (FSlateApplication::IsInitialized() && !bIdleMode)
		{
			const bool bRenderingSuspended = GEngine->IsRenderingSuspended();

			FMoviePlayerProxy::SetIsSlateThreadAllowed(false);
			FSlateApplication::Get().Tick(bRenderingSuspended ? ESlateTickType::Time : ESlateTickType::TimeAndWidgets);
			FMoviePlayerProxy::SetIsSlateThreadAllowed(true);
		}

#if WITH_ENGINE
		if (bValidateReplicatedProperties)
		{
			const bool bReplicatedPropertiesDifferent = CurrentDemoNetDriver->ComparePropertyState(PreSlateObjectStates);
			if (bReplicatedPropertiesDifferent)
			{
				UE_LOGF(LogInit, Log, "Replicated properties changed during Slate tick!");
			}
		}

		if (ConcurrentTask.GetReference())
		{
			CSV_SCOPED_SET_WAIT_STAT(ConcurrentWithSlate);

			TRACE_CPUPROFILER_EVENT_SCOPE(ConcurrentWithSlateTickTasks_Wait);
			check(ConcurrentTaskCompleteEvent);
			ConcurrentTaskCompleteEvent->Wait();
			FPlatformProcess::ReturnSynchEventToPool(ConcurrentTaskCompleteEvent);
			ConcurrentTaskCompleteEvent = nullptr;
			ConcurrentTask = nullptr;
		}
#endif

#if STATS
		// Clear any stat group notifications we have pending just in case they weren't claimed during FSlateApplication::Get().Tick
		extern CORE_API void ClearPendingStatGroups();
		ClearPendingStatGroups();
#endif

#if WITH_EDITOR && !UE_BUILD_SHIPPING
		// tick automation controller (Editor only)
		{
			SCOPED_NAMED_EVENT(Tick_AutomationController, SystemColor);
			static FName AutomationController("AutomationController");
			if (FModuleManager::Get().IsModuleLoaded(AutomationController))
			{
				FModuleManager::GetModuleChecked<IAutomationControllerModule>(AutomationController).Tick();
			}
		}
#endif

#if WITH_ENGINE && WITH_AUTOMATION_WORKER
		// tick automation worker
		{
			SCOPED_NAMED_EVENT(Tick_AutomationWorker, SystemColor);
			static const FName AutomationWorkerModuleName = "AutomationWorker";
			if (FModuleManager::Get().IsModuleLoaded(AutomationWorkerModuleName))
			{
				FModuleManager::GetModuleChecked<IAutomationWorkerModule>(AutomationWorkerModuleName).Tick();
			}
		}
#endif

		{
			SCOPED_NAMED_EVENT(Render_UpdateFrame, SystemColor);

			UE::RenderCommandPipe::StopRecording();

			for (FSceneInterface* Scene : GetRendererModule().GetAllocatedScenes())
			{
				ENQUEUE_RENDER_COMMAND(FScene_EndFrame)([Scene](FRHICommandListImmediate& RHICmdList)
				{
					Scene->EndFrame(RHICmdList);
				});
			}

			// tick render hardware interface
			{			
				SCOPE_CYCLE_COUNTER(STAT_RHITickTime);
				RHITick( FApp::GetDeltaTime() ); // Update RHI.
			}
		}

		// We need to set this marker before EndFrameRenderThread is enqueued. 
		// If multithreaded rendering is off, it can cause a bad ordering of game and rendering markers.
		GEngine->SetSimulationLatencyMarkerEnd(CurrentFrameCounter);

		// Disregard first few ticks for total tick time as it includes loading and such.
		if (GFrameCounter > 5)
		{
			TotalTickTime += FApp::GetDeltaTime();
		}

		// Find the objects which need to be cleaned up the next frame.
		FPendingCleanupObjects* PreviousPendingCleanupObjects = PendingCleanupObjects;
		PendingCleanupObjects = GetPendingCleanupObjects();

		// This could be perhaps moved down to get greater parallelism
		// Sync game and render/RHI threads.
		FFrameEndSync::Sync(FFrameEndSync::EFlushMode::EndFrame);

		// tick core ticker, threads & deferred commands
		{
			SCOPE_CYCLE_COUNTER(STAT_DeferredTickTime);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(DeferredTickTime);
			// Delete the objects which were enqueued for deferred cleanup before the previous frame.

			{
				SCOPED_NAMED_EVENT(DeletePendingCleanupObjects, SystemColor);

				delete PreviousPendingCleanupObjects;
#if WITH_COREUOBJECT
				DeleteLoaders(); // destroy all linkers pending delete
#endif
			}

			{
				SCOPED_NAMED_EVENT(Tick_Core, SystemColor);
				FTSTicker::GetCoreTicker().Tick(FApp::GetDeltaTime());
				FThreadManager::Get().Tick();
			}

			{
				SCOPED_NAMED_EVENT(Tick_DeferredCommands, SystemColor);
				GEngine->TickDeferredCommands();
			}
		}
		FMoviePlayerProxy::BlockingForceFinished();

#if !UE_SERVER
		// tick media framework
		static const FName MediaModuleName("Media");
		IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>(MediaModuleName);
		if (MediaModule != nullptr)
		{
			SCOPED_NAMED_EVENT(Tick_PostRender, SystemColor);
			MediaModule->TickPostRender();
		}
#endif

		{
			SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnEndFrame", CoreDelegatesColor);
			FCoreDelegates::OnEndFrame.Broadcast();
		}

		// Increment global frame counter. Once for each engine tick.
		GFrameCounter++;

		ENQUEUE_RENDER_COMMAND(FrameCounter)(
			[CurrentFrameCounter = GFrameCounter](FRHICommandListImmediate& RHICmdList)
		{
			GFrameCounterRenderThread = CurrentFrameCounter;
		});

#if WITH_ENGINE && CSV_PROFILER
		// By design, update this after incrementing GFrameCounter.  Calls PlatformMemoryHelpers::GetFrameMemoryStats, which caches
		// memory stats based on the value of GFrameCounter.  Placing this after the increment guarantees that cached memory stats are
		// canonically updated at this same point each frame, as opposed to arbitrarily in various other code paths (at least when the
		// CSV profiler is compiled in, which is true in all builds except shipping).
		UpdateCoreCsvStats_EndFrame();
#endif

		{
			SCOPED_NAMED_EVENT(Render_EndFrame, SystemColor);

			// Tick DumpGPU to start/stop frame dumps
#if WITH_ENGINE && WITH_DUMPGPU
				UE::RenderCore::DumpGPU::TickEndFrame();
#endif

#if !UE_SERVER && WITH_ENGINE
			{
				// We emit dynamic resolution's end frame right before RHI's. GEngine is going to ignore it if no BeginFrame was done.
				GEngine->EmitDynamicResolutionEvent(EDynamicResolutionStateEvent::EndFrame);
			}
#endif

			// end of RHI frame
			ENQUEUE_RENDER_COMMAND(EndFrame)(
				[CurrentFrameCounter](FRHICommandListImmediate& RHICmdList)
				{
					EndFrameRenderThread(RHICmdList, CurrentFrameCounter);
				});
		}

		// Set CPU utilization stats.
		const FCPUTime CPUTime = FPlatformTime::GetCPUTime();
		SET_FLOAT_STAT( STAT_CPUTimePct, CPUTime.CPUTimePct );
		SET_FLOAT_STAT( STAT_CPUTimePctRelative, CPUTime.CPUTimePctRelative );

		// Set the UObject count stat
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
		SET_DWORD_STAT(STAT_Hash_NumObjects, GUObjectArray.GetObjectArrayNumMinusAvailable());
#endif // !UE_BUILD_TEST && !UE_BUILD_SHIPPING
	}

	TRACE_END_FRAME(TraceFrameType_Game);

#if BUILD_EMBEDDED_APP
	static double LastSleepTime = FPlatformTime::Seconds();
	double TimeNow = FPlatformTime::Seconds();
	if (LastSleepTime > 0 && TimeNow - LastSleepTime >= CVarSecondsBeforeEmbeddedAppSleeps.GetValueOnAnyThread())
	{
		LastSleepTime = 0;
		FEmbeddedCommunication::AllowSleep(TEXT("FirstTicks"));
	}
#endif
	
#if PLATFORM_MAC
	// drain the apple async task queue from the game thread
	FMacPlatformApplicationMisc::ProcessAsyncTasks();
#endif
}


void FEngineLoop::ClearPendingCleanupObjects()
{
	FlushRenderingCommands();
	delete PendingCleanupObjects;
	PendingCleanupObjects = nullptr;
}

#endif // WITH_ENGINE


static TAutoConsoleVariable<int32> CVarLogTimestamp(
	TEXT("log.Timestamp"),
	1,
	TEXT("Defines if time is included in each line in the log file and in what form. Layout: [time][frame mod 1000]\n"
	     "  0 = Do not display log timestamps\n"
	     "  1 = Log time stamps in UTC and frame time (default) e.g. [2015.11.25-21.28.50:803][376]\n"
	     "  2 = Log timestamps in seconds elapsed since GStartTime e.g. [0130.29][420]"
	     "  3 = Log timestamps in local time and frame time e.g. [2017.08.04-17.59.50:803][420]"
	     "  4 = Log timestamps with the engine's timecode and frame time e.g. [17:59:50:18][420]"),
	ECVF_Default);


static TAutoConsoleVariable<int32> CVarLogCategory(
	TEXT("log.Category"),
	1,
	TEXT("Defines if the categoy is included in each line in the log file and in what form.\n"
	     "  0 = Do not log category\n"
	     "  2 = Log the category (default)"),
	ECVF_Default);


// Gets called any time cvars change (on the main thread)
static void CVarLogSinkFunction()
{
	{
		// for debugging
		ELogTimes::Type OldGPrintLogTimes = GPrintLogTimes;

		int32 LogTimestampValue = CVarLogTimestamp.GetValueOnGameThread();

		// Note GPrintLogTimes can be used on multiple threads but it should be no issue to change it on the fly
		switch(LogTimestampValue)
		{
			default:
			case 0: GPrintLogTimes = ELogTimes::None; break;
			case 1: GPrintLogTimes = ELogTimes::UTC; break;
			case 2: GPrintLogTimes = ELogTimes::SinceGStartTime; break;
			case 3: GPrintLogTimes = ELogTimes::Local; break;
			case 4: GPrintLogTimes = ELogTimes::Timecode; break;
		}
	}

	{
		int32 LogCategoryValue = CVarLogCategory.GetValueOnGameThread();

		// Note GPrintLogCategory can be used on multiple threads but it should be no issue to change it on the fly
		GPrintLogCategory = LogCategoryValue != 0;
	}
}


FAutoConsoleVariableSink CVarLogSink(FConsoleCommandDelegate::CreateStatic(&CVarLogSinkFunction));

static void CheckForPrintTimesOverride()
{
	// Determine whether to override the default setting for including timestamps in the log.
	FString LogTimes;
	if (GConfig->GetString( TEXT( "LogFiles" ), TEXT( "LogTimes" ), LogTimes, GEngineIni ))
	{
		if (LogTimes == TEXT( "None" ))
		{
			CVarLogTimestamp->Set((int)ELogTimes::None, ECVF_SetBySystemSettingsIni);
		}
		else if (LogTimes == TEXT( "UTC" ))
		{
			CVarLogTimestamp->Set((int)ELogTimes::UTC, ECVF_SetBySystemSettingsIni);
		}
		else if (LogTimes == TEXT( "SinceStart" ))
		{
			CVarLogTimestamp->Set((int)ELogTimes::SinceGStartTime, ECVF_SetBySystemSettingsIni);
		}
		else if (LogTimes == TEXT( "Local" ))
		{
			CVarLogTimestamp->Set((int)ELogTimes::Local, ECVF_SetBySystemSettingsIni);
		}
		else if (LogTimes == TEXT( "Timecode" ))
		{
			CVarLogTimestamp->Set((int)ELogTimes::Timecode, ECVF_SetBySystemSettingsIni);
		}
		// Assume this is a bool for backward compatibility
		else if (FCString::ToBool( *LogTimes ))
		{
			CVarLogTimestamp->Set((int)ELogTimes::UTC, ECVF_SetBySystemSettingsIni);
		}
	}

	if (FParse::Param( FCommandLine::Get(), TEXT( "LOGTIMES" ) ))
	{
		CVarLogTimestamp->Set((int)ELogTimes::UTC, ECVF_SetByCommandline);
	}
	else if (FParse::Param( FCommandLine::Get(), TEXT( "UTCLOGTIMES" ) ))
	{
		CVarLogTimestamp->Set((int)ELogTimes::UTC, ECVF_SetByCommandline);
	}
	else if (FParse::Param( FCommandLine::Get(), TEXT( "NOLOGTIMES" ) ))
	{
		CVarLogTimestamp->Set((int)ELogTimes::None, ECVF_SetByCommandline);
	}
	else if (FParse::Param( FCommandLine::Get(), TEXT( "LOGTIMESINCESTART" ) ))
	{
		CVarLogTimestamp->Set((int)ELogTimes::SinceGStartTime, ECVF_SetByCommandline);
	}
	else if (FParse::Param( FCommandLine::Get(), TEXT( "LOCALLOGTIMES" ) ))
	{
		CVarLogTimestamp->Set((int)ELogTimes::Local, ECVF_SetByCommandline);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT( "LOGTIMECODE" )))
	{
		CVarLogTimestamp->Set((int)ELogTimes::Timecode, ECVF_SetByCommandline);
	}
}

#if UE_EDITOR
//Standardize paths when deciding  if running the proper editor exe. 
void CleanUpPath(FString& InPath)
{
	//Converts to full path will also replace '\' with '/' and will collapse relative directories (C:\foo\..\bar to C:\bar)
	InPath = FPaths::ConvertRelativePathToFull(InPath);
	FPaths::RemoveDuplicateSlashes(InPath);
}

bool DoesCurrentTargetRequireUpgrade(const FString& EditorTargetFileName, FString& InTargetName)
{
	FTargetReceipt Receipt;

	if (!FPaths::FileExists(EditorTargetFileName))
	{
		return false;
	}

	// We do not return on Receipt.Read returning false, as Target upgrade requirements occur before a full target receipt can be emitted.
	// Only partial target receipts exist, of which the most important properties are BuildSettingsVersion && TargetBuildEnvironment.
	Receipt.Read(EditorTargetFileName);

	// Handle backwards compatibility before we added these properties to the receipt at 5.7.
	if (Receipt.BuildSettingsVersion.IsEmpty() || Receipt.TargetBuildEnvironment.IsEmpty())
	{
		UE_LOGF(
			LogInit,
			Log,
			"Unable to discern build settings version & build environment used for target - target may require upgrade. Target file: %ls",
			*EditorTargetFileName
		);
		return false;
	}

	InTargetName = Receipt.TargetName;
	const bool bIsShared = (Receipt.TargetBuildEnvironment == TEXT("Shared"));
	const bool bIsUpToDate = (Receipt.BuildSettingsVersion == TEXT("V7"));

	if (!bIsUpToDate && bIsShared)
	{
		UE_LOGF(
			LogInit,
			Warning,
			"Current target requires upgrade to V7, as it is not a unique build environment. File: %ls, BuildSettingsVersion: %ls, TargetBuildEnvironment: %ls",
			*EditorTargetFileName,
			*Receipt.BuildSettingsVersion,
			*Receipt.TargetBuildEnvironment
		);

		return true;
	}

	return false;
}

bool DoesCurrentExecutableMatch(const FString& EditorTargetFileName)
{
	FTargetReceipt Receipt;
	if (!FPaths::FileExists(EditorTargetFileName) || !Receipt.Read(EditorTargetFileName))
	{
		return false;
	}

	// Returning false for -Cmd matches legacy behavior; do we want to change this?
	const bool bCheckLaunch_true = true, bCheckCmd_false = false;
	return Receipt.LaunchesCurrentExecutable(bCheckLaunch_true, bCheckCmd_false);
}

bool LaunchCorrectEditorExecutable(const FString& EditorTargetFileName)
{
	// Don't allow relaunching the executable if we're running some unattended scripted process.
	if(FApp::IsUnattended())
	{
		return false;
	}

	// Figure out the executable that we should be running
	FString LaunchExecutableName;
	if(EditorTargetFileName.Len() == 0)
	{
		LaunchExecutableName = FPlatformProcess::GenerateApplicationPath(TEXT("UnrealEditor"), FApp::GetBuildConfiguration());
	}
	else
	{
		FTargetReceipt Receipt;
		if(!FPaths::FileExists(EditorTargetFileName) || !Receipt.Read(EditorTargetFileName))
		{
			return false;
		}
		LaunchExecutableName = Receipt.Launch;
	}
	CleanUpPath(LaunchExecutableName);

	// Get the current executable name. Don't allow relaunching if we're running the console app.
	FString CurrentExecutableName = FPlatformProcess::ExecutablePath();
	if(FPaths::GetBaseFilename(CurrentExecutableName).EndsWith(TEXT("-Cmd")))
	{
		return false;
	}
	CleanUpPath(CurrentExecutableName);

	// Nothing to do if they're the same
	if(FPaths::IsSamePath(LaunchExecutableName, CurrentExecutableName))
	{
		return false;
	}

	// Relaunch the correct executable
	UE_LOGF(LogInit, Display, "Running incorrect executable for target (%ls). Launching %ls instead...", *CurrentExecutableName, *LaunchExecutableName);
	FPlatformProcess::CreateProc(*IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*LaunchExecutableName), FCommandLine::GetOriginal(), true, false, false, nullptr, 0, nullptr, nullptr, nullptr);
	return true;
}
#endif

/* FEngineLoop static interface
 *****************************************************************************/

void FEngineLoop::AppInitUpToSetupOutputDevices()
{
	{
		SCOPED_BOOT_TIMING("BeginInitTextLocalization");
		BeginInitTextLocalization();
	}



	// Error history.
	FCString::Strcpy(GErrorHist, TEXT("Fatal error!" LINE_TERMINATOR_ANSI LINE_TERMINATOR_ANSI));

	// Platform specific pre-init.
	{
		SCOPED_BOOT_TIMING("FPlatformMisc::PlatformPreInit");
		FPlatformMisc::PlatformPreInit();
	}
#if WITH_APPLICATION_CORE
	{
		SCOPED_BOOT_TIMING("FPlatformApplicationMisc::PreInit");
		FPlatformApplicationMisc::PreInit();
	}
#endif

	// Keep track of start time.
	GSystemStartTime = FDateTime::Now().ToString();

	// Switch into executable's directory.
#if !defined(DISABLE_CWD_CHANGES) || DISABLE_CWD_CHANGES==0
	FPlatformProcess::SetCurrentWorkingDirectoryToBaseDir();
#endif

	{
		SCOPED_BOOT_TIMING("IFileManager::Get().ProcessCommandLineOptions()");
		// Now finish initializing the file manager after the command line is set up
		IFileManager::Get().ProcessCommandLineOptions();
	}

	FPageAllocator::Get().LatchProtectedMode();

	if (FParse::Param(FCommandLine::Get(), TEXT("purgatorymallocproxy")))
	{
		FMemory::EnablePurgatoryTests();
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("poisonmallocproxy")))
	{
		FMemory::EnablePoisonTests();
	}

#if !UE_BUILD_SHIPPING
	if (FParse::Param(FCommandLine::Get(), TEXT("BUILDMACHINE")))
	{
		GIsBuildMachine = true;
		// propagate to subprocesses, especially because some - like ShaderCompileWorker - use DDC, for which this switch matters
		FCommandLine::AddToSubprocessCommandLine(TEXT(" -buildmachine"), ECommandLineArgumentFlags::AllContexts);
	}
#endif // !UE_BUILD_SHIPPING

#if PLATFORM_WINDOWS

	// make sure that the log directory tree exists
	IFileManager::Get().MakeDirectory(*FPaths::ProjectLogDir(), true);

	// update the mini dump filename now that we have enough info to point it to the log folder even in installed builds
	FCString::Strcpy(MiniDumpFilenameW, *IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*FString::Printf(TEXT("%sunreal-v%i-%s.dmp"), *FPaths::ProjectLogDir(), FEngineVersion::Current().GetChangelist(), *FDateTime::Now().ToString())));
#endif
	{
		SCOPED_BOOT_TIMING("FPlatformOutputDevices::SetupOutputDevices");
		// Init logging to disk
		FPlatformOutputDevices::SetupOutputDevices();
	}
}

void FEngineLoop::AppInitLoggingOnly()
{
	AppInitUpToSetupOutputDevices();
}

bool FEngineLoop::AppInit()
{
	AppInitUpToSetupOutputDevices();

	{
		SCOPED_BOOT_TIMING("FConfigCacheIni::InitializeConfigSystem");
		LLM_SCOPE(ELLMTag::ConfigSystem);
		// init config system
		FConfigCacheIni::InitializeConfigSystem();
	}
	
#if WITH_EDITOR
	if (GIsEditor)
	{
		int32 NumPreallocateNames = 0;
		GConfig->GetInt(TEXT("NameTable"), TEXT("PreallocateNames"), NumPreallocateNames, GEditorIni);
		int32 PreAllocateNameMB = 0;
		GConfig->GetInt(TEXT("NameTable"), TEXT("PreallocateNameMemoryMB"), PreAllocateNameMB, GEditorIni);
		if (NumPreallocateNames || PreAllocateNameMB)
		{
			FName::Reserve(PreAllocateNameMB * 1024 * 1024, NumPreallocateNames);
		}
	}
#endif

	{
		SCOPED_BOOT_TIMING("ConfigUtilities::ApplyCVarsFromBootHotfix");
		// Apply boot hotfixes
		UE::ConfigUtilities::ApplyCVarsFromBootHotfix();
	}
	
#if PLATFORM_WINDOWS
	{
		SCOPED_BOOT_TIMING("ApplyWindowsCompatConfigRules");
		FWindowsPlatformMisc::ApplyWindowsCompatConfigRules();
	}
#endif // PLATFORM_WINDOWS

	// Apply config driven presets
	FTraceAuxiliary::InitializePresets(FCommandLine::Get());

	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::IniSystemReady);

	// Load "asap" plugin modules
	IPluginManager&  PluginManager = IPluginManager::Get();
	IProjectManager& ProjectManager = IProjectManager::Get();
	if (!ProjectManager.LoadModulesForProject(ELoadingPhase::EarliestPossible) || !PluginManager.LoadModulesForEnabledPlugins(ELoadingPhase::EarliestPossible))
	{
		return false;
	}

	FDelayedAutoRegisterHelper::RunAndClearDelayedAutoRegisterDelegates(EDelayedRegisterRunPhase::EarliestPossiblePluginsLoaded);

	{
		SCOPED_BOOT_TIMING("FPlatformStackWalk::Init");
		// Now that configs have been initialized, setup stack walking options
		FPlatformStackWalk::Init();
	}

	CheckForPrintTimesOverride();

	// Check whether the project or any of its plugins are missing or are out of date
#if UE_EDITOR && !IS_MONOLITHIC
	if(!GIsBuildMachine && !FApp::IsUnattended() && FPaths::IsProjectFilePathSet() && !PluginManager.GetPreloadBinaries())
	{
		// Check all the plugins are present
		if(!PluginManager.AreRequiredPluginsAvailable())
		{
			return false;
		}

		// Find the editor target
		FString EditorTargetFileName;
		FString DefaultEditorTarget;
		bool bIsRunningExe = false;
		GConfig->GetString(TEXT("/Script/BuildSettings.BuildSettings"), TEXT("DefaultEditorTarget"), DefaultEditorTarget, GEngineIni);

		// If we have multiple targets for the same code, then try to find a target that matches the running executable.  Otherwise
		// we use the first matching target.
		for (const FTargetInfo& Target : FDesktopPlatformModule::Get()->GetTargetsForProject(FPaths::GetProjectFilePath()))
		{
			if (Target.Type == EBuildTargetType::Editor)
			{
				const TCHAR* ArchitectureSuffix = FPlatformProcess::GetArchitectureSuffix();

				FString ScratchEditorTargetFileName;
				if (FPaths::IsUnderDirectory(Target.Path, FPlatformMisc::ProjectDir()))
				{
					ScratchEditorTargetFileName = FTargetReceipt::GetDefaultPath(FPlatformMisc::ProjectDir(), *Target.Name, FPlatformProcess::GetBinariesSubdirectory(), FApp::GetBuildConfiguration(), ArchitectureSuffix);
				}
				else if (FPaths::IsUnderDirectory(Target.Path, FPaths::EngineDir()))
				{
					ScratchEditorTargetFileName = FTargetReceipt::GetDefaultPath(*FPaths::EngineDir(), *Target.Name, FPlatformProcess::GetBinariesSubdirectory(), FApp::GetBuildConfiguration(), ArchitectureSuffix);
				}

				if (DoesCurrentExecutableMatch(ScratchEditorTargetFileName))
				{
					EditorTargetFileName = MoveTemp(ScratchEditorTargetFileName);
					bIsRunningExe = true;
					break;
				}
				else if (DefaultEditorTarget.Len() == 0 || Target.Name == DefaultEditorTarget)
				{
					if (EditorTargetFileName.Len() == 0)
					{
						EditorTargetFileName = MoveTemp(ScratchEditorTargetFileName);
					}
				}
			}
		}

		// If we're not running the correct executable for the current target, and the listed executable exists, run that instead
		if (!bIsRunningExe && LaunchCorrectEditorExecutable(EditorTargetFileName))
		{
			return false;
		}

		// Check if we need to compile
		bool bNeedCompile = false;
		GConfig->GetBool(TEXT("/Script/UnrealEd.EditorLoadingSavingSettings"), TEXT("bForceCompilationAtStartup"), bNeedCompile, GEditorPerProjectIni);
		if(FParse::Param(FCommandLine::Get(), TEXT("SKIPCOMPILE")) || FParse::Param(FCommandLine::Get(), TEXT("MULTIPROCESS")))
		{
			bNeedCompile = false;
		}
		if(!bNeedCompile)
		{
			// Check if any of the target, project or plugin modules are out of date, and the user wants to compile them.
			FString targetName;
			bool TargetRequiresUpgrade = DoesCurrentTargetRequireUpgrade(EditorTargetFileName, targetName);

			if (TargetRequiresUpgrade)
			{
				FString TargetRequiresUpgradeMessage = FString::Printf(
					TEXT("The last build used an outdated BuildSettingsVersion, or the target still requires upgrading."
						"\n\nPlease upgrade %s.Target.cs:"
						"\n\n  Update the build settings version (Installed Engine Build & Source Build):"
						"\n    DefaultBuildSettings = BuildSettingsVersion.V7;"
						"\n\n  OR"
						"\n\n  Declare a unique build environment (Source Build only):"
						"\n    BuildEnvironment = TargetBuildEnvironment.Unique;"
						"\n\nIf you have already updated the target, would you like to rebuild & open the project?"),
					*targetName
				);

				if (FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, *TargetRequiresUpgradeMessage, TEXT("Target Upgrade Required")) == EAppReturnType::No)
				{
					return false;
				}
				
				// Rebuilding should update the target receipt file, and prevent subsequent checks.
				bNeedCompile = true;
			}

			TArray<FString> IncompatibleFiles;
			ProjectManager.CheckModuleCompatibility(IncompatibleFiles);

			TArray<FString> IncompatibleEngineFiles;
			PluginManager.CheckModuleCompatibility(IncompatibleFiles, IncompatibleEngineFiles);

			if (IncompatibleFiles.Num() > 0)
			{
				// Log the modules which need to be rebuilt
				for (int Idx = 0; Idx < IncompatibleFiles.Num(); Idx++)
				{
					UE_LOGF(LogInit, Warning, "Incompatible or missing module: %ls", *IncompatibleFiles[Idx]);
				}

				// Build the error message for the dialog box
				FString ModulesList = "The following modules are missing or built with a different engine version:\n\n";

				int NumModulesToDisplay = (IncompatibleFiles.Num() <= 20)? IncompatibleFiles.Num() : 15;
				for (int Idx = 0; Idx < NumModulesToDisplay; Idx++)
				{
					ModulesList += FString::Printf(TEXT("  %s\n"), *IncompatibleFiles[Idx]);
				}
				if(IncompatibleFiles.Num() > NumModulesToDisplay)
				{
					ModulesList += FString::Printf(TEXT("  (+%d others, see log for details)\n"), IncompatibleFiles.Num() - NumModulesToDisplay);
				}

				// If we're running with -stdout, assume that we're a non interactive process and about to fail
				if (FApp::IsUnattended() || FParse::Param(FCommandLine::Get(), TEXT("stdout")))
				{
					return false;
				}

				// If there are any engine modules that need building, force the user to build through the IDE
				if(IncompatibleEngineFiles.Num() > 0)
				{
					FString CompileForbidden = ModulesList + TEXT("\nEngine modules cannot be compiled at runtime. Please build through your IDE.");
					FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *CompileForbidden, TEXT("Missing Modules"));
					return false;
				}

				// Ask whether to compile before continuing
				FString CompilePrompt = ModulesList + TEXT("\nWould you like to rebuild them now?");
				if (FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, *CompilePrompt, *FString::Printf(TEXT("Missing %s Modules"), FApp::GetProjectName())) == EAppReturnType::Yes)
				{
					bNeedCompile = true;
				}
				else
				{
					FString ProceedWithoutCompilePrompt = ModulesList + TEXT("\nWould you like to proceed anyway? Code and content that depends on these modules may not work correctly. Enter at your own risk.");
					if (FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, *ProceedWithoutCompilePrompt, *FString::Printf(TEXT("Missing %s Modules"), FApp::GetProjectName())) == EAppReturnType::No)
					{
						return false;
					}
				}
			}
			else if(EditorTargetFileName.Len() > 0 && !FPaths::FileExists(EditorTargetFileName))
			{
				// Prompt to compile. The target file isn't essential, but we 
				if (FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, *FString::Printf(TEXT("The %s file does not exist. Would you like to build the editor?"), *FPaths::GetCleanFilename(EditorTargetFileName)), TEXT("Missing target file")) == EAppReturnType::Yes)
				{
					bNeedCompile = true;
				}
			}
		}

		FEmbeddedCommunication::ForceTick(16);
		
		if(bNeedCompile)
		{
			// Try to compile it
			FFeedbackContext *Context = (FFeedbackContext*)FDesktopPlatformModule::Get()->GetNativeFeedbackContext();
			Context->BeginSlowTask(FText::FromString(TEXT("Starting build...")), true, true);
			ECompilationResult::Type CompilationResult = ECompilationResult::Unknown;
			bool bCompileResult = FDesktopPlatformModule::Get()->CompileGameProject(FPaths::RootDir(), FPaths::GetProjectFilePath(), Context, &CompilationResult);
			Context->EndSlowTask();

			// Check if we're running the wrong executable now
			if(bCompileResult && LaunchCorrectEditorExecutable(EditorTargetFileName))
			{
				return false;
			}

			// Check if we needed to modify engine files
			if (!bCompileResult && CompilationResult == ECompilationResult::FailedDueToEngineChange)
			{
				FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Engine modules are out of date, and cannot be compiled while the engine is running. Please build through your IDE."), TEXT("Missing Modules"));
				return false;
			}

			// Get a list of modules which are still incompatible
			TArray<FString> StillIncompatibleFiles;
			ProjectManager.CheckModuleCompatibility(StillIncompatibleFiles);

			TArray<FString> StillIncompatibleEngineFiles;
			PluginManager.CheckModuleCompatibility(StillIncompatibleFiles, StillIncompatibleEngineFiles);

			if(!bCompileResult || StillIncompatibleFiles.Num() > 0)
			{
				for (int Idx = 0; Idx < StillIncompatibleFiles.Num(); Idx++)
				{
					UE_LOGF(LogInit, Warning, "Still incompatible or missing module: %ls", *StillIncompatibleFiles[Idx]);
				}
				if (!FApp::IsUnattended())
				{
					FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *FString::Printf(TEXT("%s could not be compiled. Try rebuilding from source manually."), FApp::GetProjectName()), TEXT("Error"));
				}
				return false;
			}
		}
	}
#endif

	// Put the command line and config info into the suppression system (before plugins start loading)
	FLogSuppressionInterface::Get().ProcessConfigAndCommandLine();

#if WITH_APPLICATION_CORE && (PLATFORM_IOS || PLATFORM_TVOS)
	// Now that the config system is ready, init the audio system.
	[[IOSAppDelegate GetDelegate] InitializeAudioSession];
#endif

	// Show log if wanted.
	if (GLogConsole && FParse::Param(FCommandLine::Get(), TEXT("LOG")))
	{
		GLogConsole->Show(true);
	}

	
	// NOTE: This is the earliest place to init the online subsystems (via plugins)
	// Code needs GConfigFile to be valid
	// Must be after FThreadStats::StartThread();
	// Must be before Render/RHI subsystem D3DCreate() for platform services that need D3D hooks like Steam

	{
		SCOPED_BOOT_TIMING("Load pre-init plugin modules");
		UE_SCOPED_ENGINE_ACTIVITY(TEXT("Loading Plugins (PreInit)"));

		// Load "pre-init" plugin modules
		if (!ProjectManager.LoadModulesForProject(ELoadingPhase::PostConfigInit) || !PluginManager.LoadModulesForEnabledPlugins(ELoadingPhase::PostConfigInit))
		{
			return false;
		}
	}

	FEmbeddedCommunication::ForceTick(17);

	// after the above has run we now have the REQUIRED set of engine .INIs  (all of the other .INIs)
	// that are gotten from .h files' config() are not requires and are dynamically loaded when the .u files are loaded

#if !UE_BUILD_SHIPPING
	// Prompt the user for remote debugging?
	bool bPromptForRemoteDebug = false;
	GConfig->GetBool(TEXT("Engine.ErrorHandling"), TEXT("bPromptForRemoteDebugging"), bPromptForRemoteDebug, GEngineIni);
	bool bPromptForRemoteDebugOnEnsure = false;
	GConfig->GetBool(TEXT("Engine.ErrorHandling"), TEXT("bPromptForRemoteDebugOnEnsure"), bPromptForRemoteDebugOnEnsure, GEngineIni);

	if (FParse::Param(FCommandLine::Get(), TEXT("PROMPTREMOTEDEBUG")))
	{
		bPromptForRemoteDebug = true;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("PROMPTREMOTEDEBUGENSURE")))
	{
		bPromptForRemoteDebug = true;
		bPromptForRemoteDebugOnEnsure = true;
	}

	FPlatformMisc::SetShouldPromptForRemoteDebugging(bPromptForRemoteDebug);
	FPlatformMisc::SetShouldPromptForRemoteDebugOnEnsure(bPromptForRemoteDebugOnEnsure);

	// Feedback context.
	if (FParse::Param(FCommandLine::Get(), TEXT("WARNINGSASERRORS")))
	{
		GWarn->TreatWarningsAsErrors = true;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("ERRORSASWARNINGS")))
	{
		GWarn->TreatErrorsAsWarnings = true;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("SILENT")))
	{
		GIsSilent = true;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("RUNNINGUNATTENDEDSCRIPT")))
	{
		GIsRunningUnattendedScript = true;
	}

#endif // !UE_BUILD_SHIPPING

	// Print all initial startup logging
	FApp::PrintStartupLogMessages();

	// if a logging build, clear out old log files. Avoid races when multiple processes are running at once.
#if !NO_LOGGING
	if (!FParse::Param(FCommandLine::Get(), TEXT("MULTIPROCESS")))
	{
		FMaintenance::DeleteOldLogs();
	}
#endif

#if !UE_BUILD_SHIPPING
	{
		SCOPED_BOOT_TIMING("FApp::InitializeSession");
		FApp::InitializeSession();
	}
#endif

#if PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER
	// Delay initialization of FPersistentStorageManager to a point where GConfig is initialized
	FPersistentStorageManager::Get().Initialize();
#endif

	// Checks.
	static_assert(sizeof(uint8) == 1);
	static_assert(sizeof(int8) == 1);
	static_assert(sizeof(uint16) == 2);
	static_assert(sizeof(uint32) == 4);
	static_assert(sizeof(uint64) == 8);
	static_assert(sizeof(ANSICHAR) == 1);
	static_assert(sizeof(TCHAR) == 2);
	static_assert(sizeof(int16) == 2);
	static_assert(sizeof(int32) == 4);
	static_assert(sizeof(int64) == 8);
	static_assert(sizeof(bool) == 1);
	static_assert(sizeof(float) == 4);
	static_assert(sizeof(double) == 8);

	// Init list of common colors.
	GColorList.CreateColorMap();

	bool bForceSmokeTests = false;
	GConfig->GetBool(TEXT("AutomationTesting"), TEXT("bForceSmokeTests"), bForceSmokeTests, GEngineIni);
	bForceSmokeTests |= FParse::Param(FCommandLine::Get(), TEXT("bForceSmokeTests"));
	FAutomationTestFramework::Get().SetForceSmokeTests(bForceSmokeTests);

	FEmbeddedCommunication::ForceTick(18);

	// Init other systems.
	{
		SCOPED_BOOT_TIMING("FCoreDelegates::OnInit");
		FCoreDelegates::OnInit.Broadcast();
	}

	FEmbeddedCommunication::ForceTick(19);

	return true;
}

void FEngineLoop::AppPreExit( )
{
	SCOPED_BOOT_TIMING("AppPreExit");

	UE_LOGF(LogExit, Log, "Preparing to exit." );

	{
		SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnPreExit", CoreDelegatesColor);
		FCoreDelegates::OnPreExit.Broadcast();
	}

#if WITH_EDITOR
	FRemoteConfig::Flush();
#endif

	{
		SCOPED_NAMED_EVENT_TEXT("FCoreDelegates::OnExit", CoreDelegatesColor);
		FCoreDelegates::OnExit.Broadcast();
	}

	// FGenericRHIGPUFence uses GFrameNumberRenderThread to tell when a frame is finished. If such an object is added in the last frame before we
	// exit, it will never be "signaled", since nothing ever increments GFrameNumberRenderThread again. This can lead to deadlocks in code
	// which uses async tasks to wait until resources are safe to be deleted (for example, FMediaTextureResource).
	// To avoid this, we set the frame number to the maximum possible value here, before waiting for the thread pool to die. It's safe to do so
	// because FlushRenderingCommands() is called multiple times on exit before reaching this point, so there's no way the render thread has any
	// more frames in flight.
	GFrameNumberRenderThread = MAX_uint32;

	if (GIOThreadPool != nullptr)
	{
		GIOThreadPool->Destroy();
	}

#if (WITH_VERSE_VM || defined(__INTELLISENSE__)) && WITH_COREUOBJECT
	Verse::VerseVM::Shutdown();
#endif

#if WITH_ENGINE
	if ( GShaderCompilingManager )
	{
		delete GShaderCompilingManager;
		GShaderCompilingManager = nullptr;
	}
	if(GShaderCompilerStats)
	{
		delete GShaderCompilerStats;
		GShaderCompilerStats = nullptr;
	}

#if WITH_ODSC
	if (GODSCManager)
	{
		delete GODSCManager;
		GODSCManager = nullptr;
	}
#endif

#else
#if WITH_COREUOBJECT
	// Shutdown the PackageResourceManager in AppPreExit for programs that do not call FEngineLoop::Exit
	IPackageResourceManager::Shutdown();
#endif
#endif

}


void FEngineLoop::AppExit()
{
	static bool bCalledOnce;
	if (bCalledOnce)
	{
		return;
	}
	bCalledOnce = true;
	
	// when compiled WITH_ENGINE, this will happen in FEngineLoop::Exit()
#if !WITH_ENGINE
#if STATS
	FThreadStats::StopThread();
#endif
	FTaskGraphInterface::Shutdown();
#endif // WITH_ENGINE

	UE_LOGF(LogExit, Log, "Exiting.");

#if WITH_APPLICATION_CORE
	FPlatformApplicationMisc::TearDown();
#endif
	FPlatformMisc::PlatformTearDown();

	if (GConfig)
	{
		GConfig->Exit();
		delete GConfig;
		GConfig = nullptr;
	}

	if( GLog )
	{
		GLog->TearDown();
	}

	FTextLocalizationManager::TearDown();
	FInternationalization::TearDown();

	FTraceAuxiliary::Shutdown();
	
	AutoRTFM::Shutdown();
}

void FEngineLoop::PostInitRHI()
{
#if WITH_ENGINE
	TArray<uint32> PixelFormatByteWidth;
	PixelFormatByteWidth.AddUninitialized(PF_MAX);
	for (int i = 0; i < PF_MAX; i++)
	{
		PixelFormatByteWidth[i] = GPixelFormats[i].BlockBytes;
	}
	RHIPostInit(PixelFormatByteWidth);

	if (FApp::CanEverRender())
	{
		// perform an early check of hardware capabilities
		EShaderPlatform ShaderPlatform = GMaxRHIShaderPlatform;

		const UE::StereoRenderUtils::FStereoShaderAspects Aspects(ShaderPlatform);

		UE::StereoRenderUtils::LogISRInit(Aspects);
		
		// Initialize system textures
		ENQUEUE_RENDER_COMMAND(InitializeSystemTextures)([](FRHICommandListImmediate& RHICmdList)
		{
			GetRendererModule().InitializeSystemTextures(RHICmdList);
		});
	}

#endif
}

void FEngineLoop::PreInitHMDDevice()
{
#if WITH_ENGINE && !UE_SERVER
	if (!FParse::Param(FCommandLine::Get(), TEXT("nohmd")) && !FParse::Param(FCommandLine::Get(), TEXT("emulatestereo")))
	{
		// Get a list of modules that implement this feature
		FName Type = IHeadMountedDisplayModule::GetModularFeatureName();
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		TArray<IHeadMountedDisplayModule*> HMDModules = ModularFeatures.GetModularFeatureImplementations<IHeadMountedDisplayModule>(Type);

		// Check whether the user passed in an explicit HMD module on the command line
		FString ExplicitHMDName;
		bool bUseExplicitHMDName = FParse::Value(FCommandLine::Get(), TEXT("hmd="), ExplicitHMDName);

		// Iterate over modules, checking ExplicitHMDName and calling PreInit
		for (auto HMDModuleIt = HMDModules.CreateIterator(); HMDModuleIt; ++HMDModuleIt)
		{
			IHeadMountedDisplayModule* HMDModule = *HMDModuleIt;


			bool bUnregisterHMDModule = false;
			if (bUseExplicitHMDName)
			{
				TArray<FString> HMDAliases;
				HMDModule->GetModuleAliases(HMDAliases);
				HMDAliases.Add(HMDModule->GetModuleKeyName());

				bUnregisterHMDModule = true;
				for (const FString& HMDModuleName : HMDAliases)
				{
					if (ExplicitHMDName.Equals(HMDModuleName, ESearchCase::IgnoreCase))
					{
						bUnregisterHMDModule = !HMDModule->PreInit();
						break;
					}
				}
			}
			else
			{
				bUnregisterHMDModule = !HMDModule->PreInit();
			}

			if (bUnregisterHMDModule)
			{
				// Unregister modules which don't match ExplicitHMDName, or which fail PreInit
				ModularFeatures.UnregisterModularFeature(Type, HMDModule);
			}
		}
		// Note we do not disable or warn here if no HMD modules matched ExplicitHMDName, as not all HMD plugins have been loaded yet.
	}
#endif // #if WITH_ENGINE && !UE_SERVER
}

void FPreInitContext::Cleanup()
{
#if WITH_ENGINE && !UE_SERVER
	SlateRenderer = nullptr;
#endif // WITH_ENGINE && !UE_SERVER

	delete SlowTaskPtr;
	SlowTaskPtr = nullptr;
}

#undef LOCTEXT_NAMESPACE
