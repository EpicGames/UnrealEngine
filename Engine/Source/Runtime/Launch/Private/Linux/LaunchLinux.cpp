// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LaunchEngineLoop.h"
#include "UnixCommonStartup.h"

extern int32 GuardedMain( const TCHAR* CmdLine );

/**
 * Workaround function to avoid circular dependencies between Launch and CommonUnixStartup modules (see LaunchUnix.cpp)
 */
void LaunchUnix_FEngineLoop_AppExit();

int main(int argc, char *argv[])
{
	return CommonUnixMain(argc, argv, &GuardedMain, &LaunchUnix_FEngineLoop_AppExit);
}
