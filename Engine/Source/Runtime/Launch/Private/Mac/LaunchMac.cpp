// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/FeedbackContext.h"
#include "LaunchEngineLoop.h"
#include "HAL/ExceptionHandling.h"
#include "Mac/MacPlatformCrashContext.h"
#include "Mac/MacPlatformMiscEx.h"
#include "Apple/LaunchCPPToSwift.h"

#if WITH_ENGINE
	#include "Engine/Engine.h"
	#include "EngineGlobals.h"
#endif

#if WITH_EDITOR
	#include "Interfaces/IMainFrameModule.h"
	#include "ISettingsModule.h"
#endif

#include "Apple/ScopeAutoreleasePool.h"
#include <signal.h>
#include "Mac/CocoaThread.h"


static FEvent* GSwiftTriggerEvent = FPlatformProcess::GetSynchEventFromPool();

static FString GSavedCommandLine;
extern int32 GuardedMain( const TCHAR* CmdLine );
extern void LaunchStaticShutdownAfterError();
static int32 GGuardedMainErrorLevel = 0;

void LogCrashCallstack(const FMacCrashContext& Context)
{
    Context.ReportCrash();
    if (GLog)
    {
        GLog->Panic();
    }
    if (GWarn)
    {
        GWarn->Flush();
    }
    if (GError)
    {
        GError->Flush();
        GError->HandleError();
    }
}

/**
 * Minimal Crash Handler that logs the callstack
 */
void EngineMinimalCrashHandler(const FGenericCrashContext& GenericContext)
{
    LogCrashCallstack(static_cast<const FMacCrashContext&>( GenericContext ));
}

/**
 * Full crash handler that logs the callstack and calls a project-overridable crash handler client
 */
void EngineFullCrashHandler(const FGenericCrashContext& GenericContext)
{
    const FMacCrashContext& Context = static_cast<const FMacCrashContext&>( GenericContext );
    LogCrashCallstack(Context);
    Context.GenerateCrashInfoAndLaunchReporter();
}

@implementation UEAppDelegate

-(id)init
{
	NSArray* Arguments = [[NSProcessInfo processInfo] arguments];

	for (int32 Option = 1; Option < [Arguments count]; Option++)
	{
		GSavedCommandLine += TEXT(" ");
		FString Argument(Arguments[Option]);
		if (Argument.Contains(TEXT(" ")))
		{
			if (Argument.Contains(TEXT("=")))
			{
				FString ArgName;
				FString ArgValue;
				Argument.Split( TEXT("="), &ArgName, &ArgValue );
				Argument = FString::Printf( TEXT("%s=\"%s\""), *ArgName, *ArgValue );
			}
			else
			{
				Argument = FString::Printf(TEXT("\"%s\""), *Argument);
			}
		}
		GSavedCommandLine += Argument;
	}
	
#if !UE_BUILD_SHIPPING
	if (GSavedCommandLine.Contains(TEXT("metalshadervalidation")))
	{
		FPlatformMisc::SetEnvironmentVar(TEXT("MTL_SHADER_VALIDATION"), TEXT("1"));
		FPlatformMisc::SetEnvironmentVar(TEXT("MTL_SHADER_VALIDATION_DEFAULT_STATE"), TEXT("none"));
	}
	
	if (GSavedCommandLine.Contains(TEXT("metalapivalidation")))
	{
		FPlatformMisc::SetEnvironmentVar(TEXT("MTL_DEBUG_LAYER"), TEXT("1"));
	}
#endif

#if UE_GAME
	// On Mac we always want games to save files to user dir instead of inside the app bundle
	GSavedCommandLine += TEXT(" -installed");
#endif

	// convert $'s to " because Xcode swallows the " and this will allow -execcmds= to be usable from xcode
	GSavedCommandLine = GSavedCommandLine.Replace(TEXT("$"), TEXT("\""));

	self = [super init];
	return self;
}

- (void)awakeFromNib
{
#if WITH_EDITOR
	Filename = nil;
	bHasFinishedLaunching = false;
#endif
}

#if WITH_EDITOR
- (BOOL)application:(NSApplication *)theApplication openFile:(NSString *)filename
{
	if(!bHasFinishedLaunching && (GSavedCommandLine.IsEmpty() || GSavedCommandLine.Contains(FString(filename))))
	{
		if ([[NSFileManager defaultManager] fileExistsAtPath:filename])
		{
			Filename = filename;
		}
		return YES;
	}
	else if ([[NSFileManager defaultManager] fileExistsAtPath:filename])
	{
		NSString* ProjectName = [[filename stringByDeletingPathExtension] lastPathComponent];
		
		NSURL* BundleURL = [[NSRunningApplication currentApplication] bundleURL];
		
		NSWorkspaceOpenConfiguration* Configuration = [NSWorkspaceOpenConfiguration configuration];
		[Configuration setCreatesNewApplicationInstance:YES];
		[Configuration setPromptsUserIfNeeded:YES];
		[Configuration setArguments:[NSArray arrayWithObject: ProjectName]];

		[[NSWorkspace sharedWorkspace]
			openApplicationAtURL: BundleURL
			configuration: Configuration
				completionHandler:^(NSRunningApplication * _Nullable app, NSError * _Nullable error)
				{
					if (error) {
						NSLog(@"Failed to run the app: %@", error.localizedDescription);
					}
				}
		];
		
		return YES;
	}
	else
	{
		return YES;
	}
}
#endif

- (IBAction)requestQuit:(id)Sender
{
	GameThreadCall(^{
		if (GEngine)
		{
			if (GIsEditor)
			{
				if (IsRunningCommandlet())
				{
					RequestEngineExit(TEXT("Mac RequestQuit"));
				}
				else
				{
					GEngine->DeferredCommands.Add(TEXT("CLOSE_SLATE_MAINFRAME"));
				}
			}
			else
			{
				GEngine->DeferredCommands.Add(TEXT("EXIT"));
			}
		}
	}, false);
}

- (IBAction)showAboutWindow:(id)Sender
{
#if WITH_EDITOR
	GameThreadCall(^{
		if (FModuleManager::Get().IsModuleLoaded(TEXT("MainFrame")))
		{
			FModuleManager::GetModuleChecked<IMainFrameModule>(TEXT("MainFrame")).ShowAboutWindow();
		}
	}, false);
#else
	[NSApp orderFrontStandardAboutPanel:Sender];
#endif
}

#if WITH_EDITOR
- (IBAction)showPreferencesWindow:(id)Sender
{
	GameThreadCall(^{
		if (FModuleManager::Get().IsModuleLoaded(TEXT("Settings")))
		{
			FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(FName("Editor"), FName("General"), FName("Appearance"));
		}
	}, false);
}
#endif

//handler for the quit apple event used by the Dock menu
- (void)handleQuitEvent:(NSAppleEventDescriptor*)Event withReplyEvent:(NSAppleEventDescriptor*)ReplyEvent
{
    [self requestQuit:self];
}

- (NSApplicationTerminateReply)applicationShouldTerminate:(NSApplication *)Sender;
{
	if(!IsEngineExitRequested() || ([NSThread gameThread] && [NSThread gameThread] != [NSThread mainThread]))
	{
		if (!IsEngineExitRequested())
		{
			[self requestQuit:self];
		}
		return NSTerminateLater;
	}
	else
	{
		return NSTerminateNow;
	}
}

- (void) applicationWillTerminate:(NSNotification*)notification
{
	FTaskTagScope::SetTagStaticInit();
}

- (void) runGameThread:(id)Arg
{
	bool bIsBuildMachine = false;
#if !UE_BUILD_SHIPPING
	if (FParse::Param(*GSavedCommandLine, TEXT("BUILDMACHINE")))
	{
		bIsBuildMachine = true;
	}
#endif
	
#if UE_USE_SWIFT_UI_MAIN
	// wait until swift tells us to go
	GSwiftTriggerEvent->Wait();
#endif
	
#if UE_BUILD_DEBUG
	if( true && !GAlwaysReportCrash )
#else
	if( FPlatformMisc::IsDebuggerPresent() && !GAlwaysReportCrash )
#endif
	{
		// Don't use exception handling when a debugger is attached to exactly trap the crash. This does NOT check
		// whether we are the first instance or not!
		GGuardedMainErrorLevel = GuardedMain( *GSavedCommandLine );
	}
	else
	{
		FPlatformMisc::SetCrashHandler(bIsBuildMachine ? EngineMinimalCrashHandler : EngineFullCrashHandler);
		GIsGuarded = 1;
		// Run the guarded code.
		GGuardedMainErrorLevel = GuardedMain( *GSavedCommandLine );
		GIsGuarded = 0;
	}

	FEngineLoop::AppExit();

	if (GGuardedMainErrorLevel == 0)
	{
        dispatch_async(dispatch_get_main_queue(), ^{
            [NSApp terminate: nil];
        });
	}
	else
	{
		_Exit(GGuardedMainErrorLevel);
	}
}

- (void)applicationDidFinishLaunching:(NSNotification *)Notification
{
	SCOPED_AUTORELEASE_POOL;

	// Make sure we're running on a supported version of macOS. In some situations we cannot depend on the OS to perform the check for us.
	NSDictionary* InfoDictionary = [[NSBundle mainBundle] infoDictionary];
	NSString* MinimumSystemVersionString = (NSString*)InfoDictionary[@"LSMinimumSystemVersion"];
	NSOperatingSystemVersion MinimumSystemVersion = { 0 };
	NSOperatingSystemVersion CurrentSystemVersion = FMacPlatformMiscEx::GetNSOperatingSystemVersion();
	NSOperatingSystemVersion MinSupportedMacOSVersion = { 12, 0, 0 };
	NSString* MinSupportedMacOSVersionString = @"12.0.0";

	NSArray<NSString*>* VersionComponents = [MinimumSystemVersionString componentsSeparatedByString:@"."];
	MinimumSystemVersion.majorVersion = [[VersionComponents objectAtIndex:0] integerValue];
	MinimumSystemVersion.minorVersion = VersionComponents.count > 1 ? [[VersionComponents objectAtIndex:1] integerValue] : 0;
	MinimumSystemVersion.patchVersion = VersionComponents.count > 2 ? [[VersionComponents objectAtIndex:2] integerValue] : 0;

	// Make sure that the min version in Info.plist is at least 10.14.6, as that's the absolute minimum
	if (FMacPlatformMiscEx::VersionCompare(MinimumSystemVersion, MinSupportedMacOSVersion) < 0)
	{
		MinimumSystemVersion = MinSupportedMacOSVersion;
		MinimumSystemVersionString = MinSupportedMacOSVersionString;
	}

	if (FMacPlatformMiscEx::VersionCompare(CurrentSystemVersion, MinimumSystemVersion) < 0)
	{
		CFDictionaryRef SessionDictionary = CGSessionCopyCurrentDictionary();
		const bool bIsWindowServerAvailable = SessionDictionary != nullptr;
		if (bIsWindowServerAvailable)
		{
			NSAlert* AlertPanel = [NSAlert new];
			[AlertPanel setAlertStyle:NSAlertStyleCritical];
			[AlertPanel setInformativeText:[NSString stringWithFormat:@"You have macOS %d.%d.%d. The application requires macOS %@ or later.",
											(int32)CurrentSystemVersion.majorVersion, (int32)CurrentSystemVersion.minorVersion, (int32)CurrentSystemVersion.patchVersion, MinimumSystemVersionString]];
			[AlertPanel setMessageText:@"You cannot use this application with this version of macOS"];
			[AlertPanel addButtonWithTitle:@"OK"];
			[AlertPanel runModal];
			[AlertPanel release];

			CFRelease(SessionDictionary);
		}

		fprintf(stderr, "You cannot use this application with this version of macOS. You have macOS %d.%d.%d. The application requires macOS %s or later.\n",
				(int32)CurrentSystemVersion.majorVersion, (int32)CurrentSystemVersion.minorVersion, (int32)CurrentSystemVersion.patchVersion, [MinimumSystemVersionString UTF8String]);

		_Exit(1);
	}

#if WITH_EDITOR
	if (!FParse::Param(*GSavedCommandLine, TEXT("skipminspeccheck")))
	{
		if (!FPlatformMisc::IsRunningOnRecommendedMinSpecHardware())
		{
			CFDictionaryRef SessionDictionary = CGSessionCopyCurrentDictionary();
			const bool bIsWindowServerAvailable = SessionDictionary != nullptr;
			if (bIsWindowServerAvailable)
			{
				NSAlert* AlertPanel = [NSAlert new];
				[AlertPanel setAlertStyle:NSAlertStyleWarning];
				[AlertPanel setInformativeText:@"You are attempting to run Unreal Editor on hardware that falls below the recommended minimum configuration. If you choose to proceed, you may experience odd behavior like crashing due to memory constraints or issues with unsupported graphics drivers. You may disable this warning by running with \r '-skipminspeccheck'"];
				[AlertPanel setMessageText:@"Warning"];
				[AlertPanel addButtonWithTitle:@"Quit"];
				[AlertPanel addButtonWithTitle:@"Run Anyway"];
				NSModalResponse Response = [AlertPanel runModal];
				[AlertPanel release];

				CFRelease(SessionDictionary);

				if (Response == NSAlertFirstButtonReturn)
				{
					_Exit(1);
				}
			}
		}
	}
#endif // WITH_EDITOR
	
#if !IS_MONOLITHIC
	// UE-172403: dlopen crash on Ventura [13.0~13.3)
	if (FMacPlatformMiscEx::VersionCompare(CurrentSystemVersion, {13, 0, 0}) >= 0 && FMacPlatformMiscEx::VersionCompare(CurrentSystemVersion, {13, 3, 0}) < 0)
	{
		CFDictionaryRef SessionDictionary = CGSessionCopyCurrentDictionary();
		const bool bIsWindowServerAvailable = SessionDictionary != nullptr;
		NSString* const kDialogSuppressKey = @"VenturaCrashWarningDialogSuppression";
		if (![[NSUserDefaults standardUserDefaults] boolForKey:kDialogSuppressKey]
			&& bIsWindowServerAvailable)
		{
			NSAlert* AlertPanel = [NSAlert new];
			[AlertPanel setAlertStyle:NSAlertStyleCritical];
			[AlertPanel setInformativeText:@"Due to a conflict between certain versions of macOS Ventura and Unreal Editor, it is recommended to update to macOS 13.3 or later. Continuing may cause the editor to crash during load."];
			[AlertPanel setMessageText:@"Please update to latest macOS"];
			[AlertPanel setShowsSuppressionButton:YES];
			[AlertPanel addButtonWithTitle:@"Continue"];
			[AlertPanel addButtonWithTitle:@"Quit"];
			
			auto Result = [AlertPanel runModal];
			if (AlertPanel.suppressionButton.state == NSControlStateValueOn)
			{
				[[NSUserDefaults standardUserDefaults] setBool:YES forKey:kDialogSuppressKey];
			}
			[AlertPanel release];

			CFRelease(SessionDictionary);
			
			if (Result == NSAlertSecondButtonReturn)
			{
				_Exit(1);
			}
		}

		fprintf(stderr, "Due to a conflict between certain versions of macOS Ventura and Unreal Editor, it is recommended to update to macOS 13.3 or later. Continuing may cause the editor to crash during load.\n");
	}
#endif

	//install the custom quit event handler
    NSAppleEventManager* appleEventManager = [NSAppleEventManager sharedAppleEventManager];
    [appleEventManager setEventHandler:self andSelector:@selector(handleQuitEvent:withReplyEvent:) forEventClass:kCoreEventClass andEventID:kAEQuitApplication];
	
	FPlatformMisc::SetGracefulTerminationHandler();
#if !(UE_BUILD_SHIPPING && WITH_EDITOR) && WITH_EDITORONLY_DATA
	if ( FParse::Param( *GSavedCommandLine,TEXT("crashreports") ) )
	{
		GAlwaysReportCrash = true;
	}
#endif
	
#if WITH_EDITOR
	bHasFinishedLaunching = true;
	
	if(Filename != nil && !GSavedCommandLine.Contains(FString(Filename)))
	{
		GSavedCommandLine += TEXT(" ");
		FString Argument(Filename);
		if (Argument.Contains(TEXT(" ")))
		{
			if (Argument.Contains(TEXT("=")))
			{
				FString ArgName;
				FString ArgValue;
				Argument.Split( TEXT("="), &ArgName, &ArgValue );
				Argument = FString::Printf( TEXT("%s=\"%s\""), *ArgName, *ArgValue );
			}
			else
			{
				Argument = FString::Printf(TEXT("\"%s\""), *Argument);
			}
		}
		GSavedCommandLine += Argument;
	}
#endif
	
	RunGameThread(self, @selector(runGameThread:));
}

- (BOOL)applicationSupportsSecureRestorableState:(NSApplication *)app
{
    return false;
}

@end

extern bool GIsConsoleExecutable;

#if !UE_USE_SWIFT_UI_MAIN

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FTaskTagScope::SetTagNone();
	
	GIsConsoleExecutable = !FCString::Strstr(ArgV[0], TEXT(".app/Contents/MacOS/"));

	SCOPED_AUTORELEASE_POOL;
	[NSApplication sharedApplication];
	[NSApp setDelegate:[UEAppDelegate new]];
	[NSApp run];
	return GGuardedMainErrorLevel;
}

#endif

// swift may link to this even if we aren't using a Swift main
void KickoffEngine()
{
#if UE_USE_SWIFT_UI_MAIN
	FTaskTagScope::SetTagNone();

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Kicking off UE from Swift. Commandline: %s\n"), *GSavedCommandLine);
	GSwiftTriggerEvent->Trigger();
#endif	
}
