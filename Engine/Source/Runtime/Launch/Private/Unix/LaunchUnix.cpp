// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LaunchEngineLoop.h"
#include "UnixCommonStartup.h"

#ifndef DISABLE_ASAN_LEAK_DETECTOR
	#define DISABLE_ASAN_LEAK_DETECTOR 0
#endif

/*
 * We honestly leak so much this output is not super useful, so lets disable by default but if you want to re-enable disable
 * this DEFINE in LinuxToolchain.cs area. This must be defined in the main binary otherwise asan*.so will bind to this symbol first
 *
 * Also disable protect_shadow_gap due to a bug in some third party libs causing them to fail to init:
 *   https://github.com/google/sanitizers/issues/629
 */
extern "C" const char* LAUNCH_API __asan_default_options()
{
#if DISABLE_ASAN_LEAK_DETECTOR
	return "protect_shadow_gap=0 detect_container_overflow=0 detect_leaks=0";
#else
	return "protect_shadow_gap=0 detect_container_overflow=0";
#endif
}

extern int32 GuardedMain( const TCHAR* CmdLine );

/**
 * Workaround function to avoid circular dependencies between Launch and CommonUnixStartup modules.
 *
 * Other platforms call FEngineLoop::AppExit() in their main() (removed by preprocessor if compiled without engine), but on Unix we want to share a common main() in CommonUnixStartup module,
 * so not just the engine but all the programs could share this logic. Unfortunately, AppExit() practice breaks this nice approach since FEngineLoop cannot be moved outside of Launch without
 * making too many changes. Hence CommonUnixMain will call it through this function if WITH_ENGINE is defined.
 */
void LaunchUnix_FEngineLoop_AppExit()
{
	return FEngineLoop::AppExit();
}
