// Copyright Epic Games, Inc. All Rights Reserved.
	
#include "CoreMinimal.h"

#include "Misc/App.h"
#include "Misc/OutputDeviceError.h"
#include "Sanitizer/RaceDetector.h"
#include "LaunchEngineLoop.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/PlatformMallocCrash.h"
#include "Windows/WindowsHWrapper.h"
#include <shellapi.h>

#if UE_BUILD_DEBUG
#include <crtdbg.h>
#endif

#if USING_ADDRESS_SANITISER

#include <sanitizer/asan_interface.h>

DEFINE_LOG_CATEGORY_STATIC(LogASan, Log, All);

static void ASanErrorCallback(const char* ErrorStr)
{
	if (GIsCriticalError)
	{
		UE_LOGF(LogASan, Warning, "Critical error raised prior to ASan error; engine memory may be unstable in this state!");
	}
	/* Marking the end of sanitizer report for Gauntlet. */
	UE_LOGF(LogASan, Fatal, "ASan Error: %ls\nEnd of Address Sanitizer report", ANSI_TO_TCHAR(ErrorStr));
}

#endif

DEFINE_LOG_CATEGORY_STATIC(LogLaunchWindows, Log, All);

extern int32 GuardedMain( const TCHAR* CmdLine );
extern void LaunchStaticShutdownAfterError();

// http://developer.download.nvidia.com/devzone/devcenter/gamegraphics/files/OptimusRenderingPolicies.pdf
// The following line is to favor the high performance NVIDIA GPU if there are multiple GPUs
// Has to be .exe module to be correctly detected.
extern "C" { _declspec(dllexport) uint32 NvOptimusEnablement = 0x00000001; }

// And the AMD equivalent
// Also has to be .exe module to be correctly detected.
extern "C" { _declspec(dllexport) uint32 AmdPowerXpressRequestHighPerformance = 0x00000001; }

#if !defined(D3D12_CORE_ENABLED)
	#define D3D12_CORE_ENABLED 0
#endif

#if D3D12_CORE_ENABLED
// Add Agility SDK symbols for Windows
#include "UnrealAgilitySDKLink.inl"
#endif

/** Whether we should pause before exiting. used by UCC */
bool		GShouldPauseBeforeExit;

/**
 * Handler for CRT parameter validation. Triggers error
 *
 * @param Expression - the expression that failed crt validation
 * @param Function - function which failed crt validation
 * @param File - file where failure occured
 * @param Line - line number of failure
 * @param Reserved - not used
 */
void InvalidParameterHandler(const TCHAR* Expression,
							 const TCHAR* Function, 
							 const TCHAR* File, 
							 uint32 Line, 
							 uintptr_t Reserved)
{
	UE_LOGF(LogLaunchWindows, Fatal,"SECURE CRT: Invalid parameter detected.\nExpression: %ls Function: %ls. File: %ls Line: %d\n", 
		Expression ? Expression : TEXT("Unknown"), 
		Function ? Function : TEXT("Unknown"), 
		File ? File : TEXT("Unknown"), 
		Line );
}

/**
 * Setup the common debug settings 
 */
void SetupWindowsEnvironment( void )
{
	// all crt validation should trigger the callback
	_set_invalid_parameter_handler(InvalidParameterHandler);

#if UE_BUILD_DEBUG
	// Disable the message box for assertions and just write to debugout instead
	_CrtSetReportMode( _CRT_ASSERT, _CRTDBG_MODE_DEBUG );
	// don't fill buffers with 0xfd as we make assumptions for FNames st we only use a fraction of the entire buffer
	_CrtSetDebugFillThreshold( 0 );
#endif
}
/**
 * The inner exception handler catches crashes/asserts in native C++ code and is the only way to get the correct callstack
 * when running a 64-bit executable. However, XAudio2 doesn't always like this and it may result in no sound.
 */
#ifdef _WIN64
	bool GEnableInnerException = true;
#else
	bool GEnableInnerException = false;
#endif

/**
 * The inner exception handler catches crashes/asserts in native C++ code and is the only way to get the correct callstack
 * when running a 64-bit executable. However, XAudio2 doesn't like this and it may result in no sound.
 */
LAUNCH_API int32 GuardedMainWrapper( const TCHAR* CmdLine )
{
	int32 ErrorLevel = 0;
	if ( GEnableInnerException )
	{
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	 	__try
#endif
		{
			// Run the guarded code.
			ErrorLevel = GuardedMain( CmdLine );
		}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__except( FPlatformMisc::GetCrashHandlingType() == ECrashHandlingType::Default ? (ReportCrash( GetExceptionInformation()), EXCEPTION_CONTINUE_SEARCH) : EXCEPTION_CONTINUE_SEARCH )
		{
			// Deliberately do nothing but avoid warning C6322: Empty _except block.
			(void)0;
		}
#endif
	}
	else
	{
		// Run the guarded code.
		ErrorLevel = GuardedMain( CmdLine );
	}
	return ErrorLevel;
}

/**
 * The command-line invocation string, processed using the standard Windows CommandLineToArgvW implementation.
 * This need to be a static global to avoid object unwinding errors in WinMain when SEH is enabled.
 */
static FString GSavedCommandLine;

bool ProcessCommandLine()
{
	int argc = 0;
	LPWSTR* argv = ::CommandLineToArgvW(::GetCommandLineW(), &argc);
	if (argv != nullptr)
	{
		// Reconstruct our command line string in a format suitable for consumption by the FParse class
		GSavedCommandLine = "";
		for (int32 Option = 1; Option < argc; Option++)
		{
			GSavedCommandLine += TEXT(" ");
			
			// Inject quotes in the correct position to preserve arguments containing whitespace
			FString Temp = argv[Option];
			if (Temp.Contains(TEXT(" ")))
			{
				int32 Quote = 0;
				if(Temp.StartsWith(TEXT("-")))
				{
					int32 Separator;
					if (Temp.FindChar('=', Separator))
					{
						Quote = Separator + 1;
					}
				}
				Temp = Temp.Left(Quote) + TEXT("\"") + Temp.Mid(Quote) + TEXT("\"");
			}
			GSavedCommandLine += Temp;
		}
		
		// Free memory allocated for CommandLineToArgvW() arguments
		::LocalFree(argv);
		return true;
	}
	
	return false;
}

LAUNCH_API int32 LaunchWindowsStartup( HINSTANCE hInInstance, HINSTANCE hPrevInstance, char*, int32 nCmdShow, const TCHAR* CmdLine )
{
	TRACE_BOOKMARK(TEXT("WinMain.Enter"));
#if USING_ADDRESS_SANITISER
	__asan_set_error_report_callback(ASanErrorCallback);
#endif

	// Setup common Windows settings
	SetupWindowsEnvironment();

	int32 ErrorLevel			= 0;
	hInstance				= hInInstance;

	if (!CmdLine)
	{
		CmdLine = ::GetCommandLineW();

		// Attempt to process the command-line arguments using the standard Windows implementation
		// (This ensures behavior parity with other platforms where argc and argv are used.)
		if ( ProcessCommandLine() )
		{
			CmdLine = *GSavedCommandLine;
		}
	}

	// If we're running in unattended mode, make sure we never display error dialogs if we crash.
	if ( FParse::Param( CmdLine, TEXT("unattended") ) )
	{
		SetErrorMode(SEM_FAILCRITICALERRORS | SEM_NOGPFAULTERRORBOX | SEM_NOOPENFILEERRORBOX);
	}

	if ( FParse::Param( CmdLine,TEXT("crashreports") ) )
	{
		GAlwaysReportCrash = true;
	}

	uint32 LogicalCoreAffinity = 0;
	uint32 PhysicalCoreAffinity = 0;
	FParse::Value(CmdLine, TEXT("-processaffinity="), LogicalCoreAffinity);
	FParse::Value(CmdLine, TEXT("-processaffinityphysical="), PhysicalCoreAffinity);

	if (LogicalCoreAffinity > 0)
	{
		FWindowsPlatformProcess::SetProcessAffinity(LogicalCoreAffinity, false);
	}
	else if (PhysicalCoreAffinity > 0)
	{
		FWindowsPlatformProcess::SetProcessAffinity(PhysicalCoreAffinity, true);
	}

	bool bNoExceptionHandler = FParse::Param(CmdLine,TEXT("noexceptionhandler"));
	(void)bNoExceptionHandler;

	bool bIgnoreDebugger = FParse::Param(CmdLine, TEXT("IgnoreDebugger"));
	(void)bIgnoreDebugger;

	bool bIsDebuggerPresent = FPlatformMisc::IsDebuggerPresent() && !bIgnoreDebugger;
	(void)bIsDebuggerPresent;

	// Using the -noinnerexception parameter will disable the exception handler within native C++, which is call from managed code,
	// which is called from this function.
	// The default case is to have three wrapped exception handlers 
	// Native: WinMain() -> Native: GuardedMainWrapper().
	// The inner exception handler in GuardedMainWrapper() catches crashes/asserts in native C++ code and is the only way to get the
	// correct callstack when running a 64-bit executable. However, XAudio2 sometimes (?) don't like this and it may result in no sound.
#ifdef _WIN64
	if ( FParse::Param(CmdLine,TEXT("noinnerexception")) || FApp::IsBenchmarking() || bNoExceptionHandler)
	{
		GEnableInnerException = false;
	}
#endif	

	// When we're running embedded, assume that the outer application is going to be handling crash reporting
#if UE_BUILD_DEBUG
	if (GUELibraryOverrideSettings.bIsEmbedded || !GAlwaysReportCrash)
#else
	if (GUELibraryOverrideSettings.bIsEmbedded || bNoExceptionHandler || (bIsDebuggerPresent && !GAlwaysReportCrash))
#endif
	{
		// Don't use exception handling when a debugger is attached to exactly trap the crash. This does NOT check
		// whether we are the first instance or not!
		ErrorLevel = GuardedMain( CmdLine );
	}
	else
	{
		// Use structured exception handling to trap any crashes, walk the the stack and display a crash dialog box.
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__try
#endif
 		{
			GIsGuarded = 1;
			// Run the guarded code.
			ErrorLevel = GuardedMainWrapper( CmdLine );
			GIsGuarded = 0;
		}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__except( FPlatformMisc::GetCrashHandlingType() == ECrashHandlingType::Default
				? ( GEnableInnerException ? EXCEPTION_EXECUTE_HANDLER : ReportCrash(GetExceptionInformation()) )
				: EXCEPTION_CONTINUE_SEARCH )	
		{
			// Crashed.
			ErrorLevel = 1;
			if(GError)
			{
				GError->HandleError();
			}
			LaunchStaticShutdownAfterError();
			FPlatformMallocCrash::Get().PrintPoolsUsage();
			FPlatformMisc::RequestExit( true, TEXT("LaunchWindowsStartup.ExceptionHandler"));
		}
#endif
	}

	TRACE_BOOKMARK(TEXT("WinMain.Exit"));

	return ErrorLevel;
}

LAUNCH_API void LaunchWindowsShutdown()
{
	// Final shut down.
	FEngineLoop::AppExit();

	// pause if we should
	if (GShouldPauseBeforeExit)
	{
		Sleep(INFINITE);
	}
}

#include <Containers/SharedString.h>

int32 WINAPI WinMain(_In_ HINSTANCE hInInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ char* pCmdLine, _In_ int32 nCmdShow)
{
	int32 Result;
	{
#if USING_INSTRUMENTATION

		if (!UE::Sanitizer::RaceDetector::Initialize())
		{
			return 1;
		}

		ON_SCOPE_EXIT
		{
			if (!UE::Sanitizer::RaceDetector::Shutdown())
			{
				Result = 1;
			}
		};

#endif

		Result = LaunchWindowsStartup(hInInstance, hPrevInstance, pCmdLine, nCmdShow, nullptr);
		LaunchWindowsShutdown();
	}
	return Result;
}

