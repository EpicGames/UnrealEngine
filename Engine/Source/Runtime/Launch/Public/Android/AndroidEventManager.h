// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "Templates/Atomic.h"

#if USE_ANDROID_EVENTS

struct ANativeWindow;

DECLARE_LOG_CATEGORY_EXTERN(LogAndroidEvents, Log, All);

enum EAppEventState
{
	APP_EVENT_STATE_WINDOW_CREATED,
	APP_EVENT_STATE_WINDOW_RESIZED,
	APP_EVENT_STATE_WINDOW_CHANGED,
	APP_EVENT_STATE_WINDOW_DESTROYED,
	APP_EVENT_STATE_WINDOW_REDRAW_NEEDED,
	APP_EVENT_STATE_ON_DESTROY,
	APP_EVENT_STATE_ON_PAUSE,
	APP_EVENT_STATE_ON_RESUME,
	APP_EVENT_STATE_ON_STOP,
	APP_EVENT_STATE_ON_START,
	APP_EVENT_STATE_WINDOW_LOST_FOCUS,
	APP_EVENT_STATE_WINDOW_GAINED_FOCUS,
	APP_EVENT_STATE_SAVE_STATE,
	APP_EVENT_STATE_APP_SUSPENDED,
	APP_EVENT_STATE_APP_ACTIVATED,
	APP_EVENT_STATE_SAFE_ZONE_UPDATED,
	APP_EVENT_RUN_CALLBACK,
	APP_EVENT_MAX_EVENTS,
	APP_EVENT_STATE_INVALID = -1,
};

struct FAppEventData
{
	FAppEventData() : WindowWidth(-1), WindowHeight(-1)
	{}
	FAppEventData(TFunction<void(void)> CallbackFuncIN) : WindowWidth(-1), WindowHeight(-1), CallbackFunc(MoveTemp(CallbackFuncIN))
	{}


	FAppEventData(ANativeWindow* WindowIn);

	int32 WindowWidth;
	int32 WindowHeight;

	TFunction<void(void)> CallbackFunc;
};

struct FAppEventPacket
{
	EAppEventState State;
	FAppEventData Data;

	FAppEventPacket():
		State(APP_EVENT_STATE_INVALID)
	{	}
};

class FAppEventManager
{
public:
	static FAppEventManager* GetInstance();
	
	void Tick();
	void EnqueueAppEvent(EAppEventState InState, FAppEventData&& InData = FAppEventData());
	void SetEventHandlerEvent(FEvent* InEventHandlerEvent);
	// These are called directly from the android event thread.
	void HandleWindowCreated_EventThread(void* InWindow);
	void HandleWindowClosed_EventThread();
	bool IsGamePaused();
	bool IsGameInFocus();
	bool WaitForEventInQueue(EAppEventState InState, double TimeoutSeconds);

	void SetEmptyQueueHandlerEvent(FEvent* InEventHandlerEvent);
	void TriggerEmptyQueue();

	void SetAppIsActive(bool bIsActive);

	void PauseAudio();
	void ResumeAudio();
	static void ReleaseMicrophone(bool shuttingDown);

	// When the game thread is stalled by operations like switching to background,
	// GameThreadTicker can be used to resume the game thread for 1 frame and 
	// stall it again in the subsequent frame.
	// If a wait call is behind a GameThreadTicker object on another thread, the GT will
	// be resumed until the wait returns and the life time of the object ends.
	class GameThreadTicker
	{
	public:

		GameThreadTicker()
		{
			FAppEventManager::GetInstance()->ResumePausedGameThread();
		}

		~GameThreadTicker()
		{
			FAppEventManager::GetInstance()->KeepGameThreadPaused();
		}
	};

	friend class GameThreadTicker;

protected:
	FAppEventManager();

	void ResumePausedGameThread();
	void KeepGameThreadPaused();

private:
	FAppEventPacket DequeueAppEvent();

	void ExecWindowCreated();
	void ExecWindowResized();
	
	static void OnScaleFactorChanged(IConsoleVariable* CVar);

	static FAppEventManager* sInstance;

	pthread_mutex_t QueueMutex;			//@todo android: can probably get rid of this now that we're using an mpsc queue
	TQueue<FAppEventPacket, EQueueMode::Mpsc> Queue;
	
	FEvent* EventHandlerEvent;			// triggered every time the event handler thread fires off an event
	FEvent* EmptyQueueHandlerEvent;		// triggered every time the queue is emptied

	//states
	bool FirstInitialized;
	bool bCreateWindow;

	bool bWindowInFocus;
	bool bSaveState;
	bool bAudioPaused;

	bool bHaveWindow;
	bool bHaveGame;
	TAtomic<bool> bRunning;
	
	TAtomic<bool> bTempRunningTriggered;// Flag to identify if the temporary running has been trigged so that APP_EVENT_STATE_APP_ACTIVATED will be only enqueued once.
	TAtomic<bool> bAppIsActive;			// Synced with bAppIsActive_EventThread in LaunchAndroid
};

bool IsInAndroidEventThread();

#endif