// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if USE_ANDROID_JNI
#include <jni.h>
#include <android/log.h>

extern JavaVM* GJavaVM;

DECLARE_MULTICAST_DELEGATE_SixParams(FOnActivityResult, JNIEnv *, jobject, jobject, jint, jint, jobject);

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSafetyNetAttestationResult, bool, const FString&, int32);

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRouteServiceIntent, const FString&, const FString&);

// Define all the Java classes/methods that the game will need to access to
class FJavaWrapper
{
public:

	// Nonstatic methods
	static jclass GameActivityClassID;
	static jobject GameActivityThis;
	static jmethodID AndroidThunkJava_ShowConsoleWindow;
    static jmethodID AndroidThunkJava_ShowVirtualKeyboardInputDialog;
    static jmethodID AndroidThunkJava_HideVirtualKeyboardInputDialog;
	static jmethodID AndroidThunkJava_ShowVirtualKeyboardInput;
	static jmethodID AndroidThunkJava_HideVirtualKeyboardInput;
	static jmethodID AndroidThunkJava_LaunchURL;
	static jmethodID AndroidThunkJava_GetAssetManager;
	static jmethodID AndroidThunkJava_Minimize;
    static jmethodID AndroidThunkJava_ClipboardCopy;
    static jmethodID AndroidThunkJava_ClipboardPaste;
	static jmethodID AndroidThunkJava_ForceQuit;
	static jmethodID AndroidThunkJava_GetFontDirectory;
	static jmethodID AndroidThunkJava_Vibrate;
	static jmethodID AndroidThunkJava_IsMusicActive;
	static jmethodID AndroidThunkJava_IsScreensaverEnabled;
	static jmethodID AndroidThunkJava_KeepScreenOn;
	static jmethodID AndroidThunkJava_InitHMDs;
	static jmethodID AndroidThunkJava_DismissSplashScreen;
	static jmethodID AndroidThunkJava_ShowProgressDialog;
	static jmethodID AndroidThunkJava_UpdateProgressDialog;
	static jmethodID AndroidThunkJava_GetInputDeviceInfo;
	static jmethodID AndroidThunkJava_SetInputDeviceVibrators;
	static jmethodID AndroidThunkJava_IsGamepadAttached;
	static jmethodID AndroidThunkJava_HasMetaDataKey;
	static jmethodID AndroidThunkJava_GetMetaDataBoolean;
	static jmethodID AndroidThunkJava_GetMetaDataInt;
	static jmethodID AndroidThunkJava_GetMetaDataLong;
	static jmethodID AndroidThunkJava_GetMetaDataFloat;
	static jmethodID AndroidThunkJava_GetMetaDataString;
	static jmethodID AndroidThunkJava_IsOpenXRImmersiveHMDApplication;
	static jmethodID AndroidThunkJava_IsOculusMobileApplication;
	static jmethodID AndroidThunkJava_ShowHiddenAlertDialog;
	static jmethodID AndroidThunkJava_LocalNotificationScheduleAtTime;
	static jmethodID AndroidThunkJava_LocalNotificationClearAll;
	static jmethodID AndroidThunkJava_LocalNotificationExists;
	static jmethodID AndroidThunkJava_LocalNotificationGetLaunchNotification;
	static jmethodID AndroidThunkJava_LocalNotificationDestroyIfExists;
	static jmethodID AndroidThunkJava_GetNetworkConnectionType;
	static jmethodID AndroidThunkJava_GetAndroidId;
	static jmethodID AndroidThunkJava_ShareURL;
	static jmethodID AndroidThunkJava_LaunchPackage;
	static jmethodID AndroidThunkJava_IsPackageInstalled;
	static jmethodID AndroidThunkJava_SendBroadcast;
	static jmethodID AndroidThunkJava_HasIntentExtrasKey;
	static jmethodID AndroidThunkJava_GetIntentExtrasBoolean;
	static jmethodID AndroidThunkJava_GetIntentExtrasInt;
	static jmethodID AndroidThunkJava_GetIntentExtrasString;
	static jmethodID AndroidThunkJava_GetIntentDataAsString;
	static jmethodID AndroidThunkJava_SetSustainedPerformanceMode;
	static jmethodID AndroidThunkJava_PushSensorEvents;
	static jmethodID AndroidThunkJava_SetOrientation;
	static jmethodID AndroidThunkJava_HasSharedPreference;
	static jmethodID AndroidThunkJava_SetSharedPreferenceBoolean;
	static jmethodID AndroidThunkJava_GetSharedPreferenceBoolean;
	static jmethodID AndroidThunkJava_SetSharedPreferenceInt;
	static jmethodID AndroidThunkJava_GetSharedPreferenceInt;
	static jmethodID AndroidThunkJava_SetSharedPreferenceString;
	static jmethodID AndroidThunkJava_GetSharedPreferenceString;
	static jmethodID AndroidThunkJava_DeleteSharedPreference;
	static jmethodID AndroidThunkJava_DeleteSharedPreferenceGroup;
	static jmethodID AndroidThunkJava_GetCacheDir;


	// Screen capture/recording permission
	static jmethodID AndroidThunkJava_IsScreenCaptureDisabled;
	static jmethodID AndroidThunkJava_DisableScreenCapture;

	// Native view click filtering
	static jmethodID AndroidThunkJava_HasExtraIgnoreView;

	// InputDeviceInfo member field ids
	static jclass InputDeviceInfoClass;
	static jfieldID InputDeviceInfo_VendorId;
	static jfieldID InputDeviceInfo_ProductId;
	static jfieldID InputDeviceInfo_ControllerId;
	static jfieldID InputDeviceInfo_Name;
	static jfieldID InputDeviceInfo_Descriptor;
	static jfieldID InputDeviceInfo_FeedbackMotorCount;
	static jfieldID InputDeviceInfo_IsExternal;

	// IDs related to google play services
	static jclass GoogleServicesClassID;
	static jobject GoogleServicesThis;
	static jmethodID AndroidThunkJava_ShowAdBanner;
	static jmethodID AndroidThunkJava_HideAdBanner;
	static jmethodID AndroidThunkJava_CloseAdBanner;
	static jmethodID AndroidThunkJava_LoadInterstitialAd;
	static jmethodID AndroidThunkJava_IsInterstitialAdAvailable;
	static jmethodID AndroidThunkJava_IsInterstitialAdRequested;
	static jmethodID AndroidThunkJava_ShowInterstitialAd;
	static jmethodID AndroidThunkJava_GetAdvertisingId;
	
	// Optionally added if GCM plugin (or other remote notification system) enabled
	static jmethodID AndroidThunkJava_RegisterForRemoteNotifications;
	static jmethodID AndroidThunkJava_UnregisterForRemoteNotifications;
	static jmethodID AndroidThunkJava_IsAllowedRemoteNotifications;

	// In app purchase functionality
	static jclass JavaStringClass;
	static jmethodID AndroidThunkJava_IapSetupService;
	static jmethodID AndroidThunkJava_IapQueryInAppPurchases;
	static jmethodID AndroidThunkJava_IapBeginPurchase;
	static jmethodID AndroidThunkJava_IapIsAllowedToMakePurchases;
	static jmethodID AndroidThunkJava_IapQueryExistingPurchases;
	static jmethodID AndroidThunkJava_IapAcknowledgePurchase;
	static jmethodID AndroidThunkJava_IapConsumePurchase;
	static jmethodID AndroidThunkJava_IapGetBillingCountryCode;
	static jmethodID AndroidThunkJava_IapGetIsAlternativeBillingOnlyAvailable;
	static jmethodID AndroidThunkJava_IapShowAlternativeBillingOnlyInformationDialog;

	// SurfaceView functionality for view scaling on some devices
	static jmethodID AndroidThunkJava_UseSurfaceViewWorkaround;
	static jmethodID AndroidThunkJava_SetDesiredViewSize;
	static jmethodID AndroidThunkJava_VirtualInputIgnoreClick;
	static jmethodID AndroidThunkJava_RestartApplication;
	
	// Screen refresh rate
	static jmethodID AndroidThunkJava_GetNativeDisplayRefreshRate;
	static jmethodID AndroidThunkJava_SetNativeDisplayRefreshRate;
	static jmethodID AndroidThunkJava_GetSupportedNativeDisplayRefreshRates;

	// Motion controls
	static jmethodID AndroidThunkJava_EnableMotion;

	// Network Connection Listener
	static jmethodID AndroidThunkJava_AddNetworkListener;
	static jmethodID AndroidThunkJava_RemoveNetworkListener;

	// member fields for getting the launch notification
	static jclass LaunchNotificationClass;
	static jfieldID LaunchNotificationUsed;
	static jfieldID LaunchNotificationEvent;
	static jfieldID LaunchNotificationFireDate;

	// method and classes for thread name change
	static jclass ThreadClass;
	static jmethodID CurrentThreadMethod;
	static jmethodID SetNameMethod;

	// WifiManager's Multicastlock handling
	static jmethodID AndroidThunkJava_AcquireWifiManagerMulticastLock;
	static jmethodID AndroidThunkJava_ReleaseWifiManagerMulticastLock;

	// audio focus
	static jmethodID AndroidThunkJava_OnAudioFocusChangedListenersUpdated;

	/**
	 * Find all known classes and methods
	 */
	static void FindClassesAndMethods(JNIEnv* Env);

	/**
	 * Helper wrapper functions around the JNIEnv versions with NULL/error handling
	 */
	static jclass FindClass(JNIEnv* Env, const ANSICHAR* ClassName, bool bIsOptional);
	static jclass FindClassGlobalRef(JNIEnv* Env, const ANSICHAR* ClassName, bool bIsOptional);
	static jmethodID FindMethod(JNIEnv* Env, jclass Class, const ANSICHAR* MethodName, const ANSICHAR* MethodSignature, bool bIsOptional);
	static jmethodID FindStaticMethod(JNIEnv* Env, jclass Class, const ANSICHAR* MethodName, const ANSICHAR* MethodSignature, bool bIsOptional);
	static jfieldID FindField(JNIEnv* Env, jclass Class, const ANSICHAR* FieldName, const ANSICHAR* FieldType, bool bIsOptional);

	static void CallVoidMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...);
	static jobject CallObjectMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...);
	static int32 CallIntMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...);
	static int64 CallLongMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...);
	static float CallFloatMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...);
	static double CallDoubleMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...);	
	static bool CallBooleanMethod(JNIEnv* Env, jobject Object, jmethodID Method, ...);

	static void CallStaticVoidMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...);
	static jobject CallStaticObjectMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...);
	static int32 CallStaticIntMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...);
	static int64 CallStaticLongMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...);
	static float CallStaticFloatMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...);
	static double CallStaticDoubleMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...);
	static bool CallStaticBooleanMethod(JNIEnv* Env, jclass Clazz, jmethodID Method, ...);

	// Delegate that can be registered to that is called when an activity is finished
	static FOnActivityResult OnActivityResultDelegate;

	// Delegate that can be registered to that is called when an SafetyNet Attestation is finished
	static FOnSafetyNetAttestationResult OnSafetyNetAttestationResultDelegate;

	// Delegate that can be registered to be called when a service intent is received
	static FOnRouteServiceIntent OnRouteServiceIntentDelegate;

private:

	/** Find GooglePlay "game services" classes and methods */
	static void FindGooglePlayMethods(JNIEnv* Env);
	/** Find GooglePlay billing classes and methods */
	static void FindGooglePlayBillingMethods(JNIEnv* Env);

	// Setup communication with wrapper apps
	static void SetupEmbeddedCommunication(JNIEnv* Env);
};
#endif
