// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_ENGINE
	#include "UnrealEngine.h"
#endif

class FEngineService;
class FTraceService;
class FPendingCleanupObjects;
class ISessionService;
class FSlateRenderer;

struct FScopedSlowTask;

struct FPreInitContext
{
	bool bDumpEarlyConfigReads = false;
	bool bDumpEarlyPakFileReads = false;
	bool bForceQuitAfterEarlyReads = false;
	bool bWithConfigPatching = false;
	UE_DEPRECATED(5.5, "Use gc.MaxObjectsNotConsideredByGC=0 ini setting instead")
	bool bDisableDisregardForGC = false;
	bool bHasEditorToken = false;
	bool bIsRegularClient = false;
	UE_DEPRECATED(5.1, "Use bIsPossiblyUnrecognizedCommandlet instead")
	bool bTokenDoesNotHaveDash = false;
	bool bIsPossiblyUnrecognizedCommandlet = false;

	FString Token;
	FString CommandletCommandLine;

	FScopedSlowTask* SlowTaskPtr = nullptr;

	void Cleanup();

#if WITH_ENGINE && !UE_SERVER
	TSharedPtr<FSlateRenderer> SlateRenderer;
#endif // WITH_ENGINE && !UE_SERVER
};

/**
 * Implements the main engine loop.	
 */
class FEngineLoop
#if WITH_ENGINE
	: public IEngineLoop
#endif
{
public:

	/** Default constructor. */
	FEngineLoop();

	virtual ~FEngineLoop() = default;

public:

	/**
	 * Pre-Initialize the main loop, and generates the commandline from standard ArgC/ArgV from main().
	 *
	 * @param ArgC The number of strings in ArgV.
	 * @param ArgV The command line parameters (ArgV[0] is expected to be the executable name).
	 * @param AdditionalCommandLine Optional string to append to the command line (after ArgV is put together).
	 * @return Returns the error level, 0 if successful and > 0 if there were errors.
	 */ 
	int32 PreInit(int32 ArgC, TCHAR* ArgV[], const TCHAR* AdditionalCommandline = nullptr);

	/**
	 * Pre-Initialize the main loop - parse command line, sets up GIsEditor, etc.
	 *
	 * @param CmdLine The command line.
	 * @return The error level; 0 if successful, > 0 if there were errors.
	 */ 
	int32 PreInit(const TCHAR* CmdLine);
	
	/** First part of PreInit. */
	int32 PreInitPreStartupScreen(const TCHAR* CmdLine);

	/** Second part of PreInit. */
	int32 PreInitPostStartupScreen(const TCHAR* CmdLine);

	/** Load all modules needed before Init. */ 
	void LoadPreInitModules();

	/** Load core modules. */
	bool LoadCoreModules();

	/** Allow override of the project module loaded from project file */
	void OverrideProjectModule(const FString& InOriginalProjectModuleName, const FString& InReplacementProjectModuleName);

	/** Clean up PreInit context. */
	void CleanupPreInitContext();

#if WITH_ENGINE
	
	/** Load all core modules needed at startup time. */
	bool LoadStartupCoreModules();
	
	/** Load all modules needed at startup time. */
	bool LoadStartupModules();

	/**
	 * Initialize the main loop (the rest of the initialization).
	 *
	 * @return The error level; 0 if successful, > 0 if there were errors.
	 */ 
	virtual int32 Init() override;

	/** Initialize the timing options from the command line. */ 
	void InitTime();

	/** Performs shut down. */
	void Exit();

	/** Whether the engine should operate in an idle mode that uses no CPU or GPU time. */
	bool ShouldUseIdleMode() const;

	/** Advances the main loop. */
	virtual void Tick() override;

	/** Removes references to any objects pending cleanup by deleting them. */
	virtual void ClearPendingCleanupObjects() override;

#endif // WITH_ENGINE

	/** RHI post-init initialization */
	static void PostInitRHI();

	/** Pre-init HMD device (if necessary). */
	static void PreInitHMDDevice();

public:

	/** Initializes the application. */
	static bool AppInit();

	/** Initializes the application sufficiently far to record a log, but then returns without any further setup. */
	static void AppInitLoggingOnly();

	/**
	 * Prepares the application for shutdown.
	 *
	 * This function is called from within guarded exit code, only during non-error exits.
	 */
	static void AppPreExit();

	/**
	 * Shuts down the application.
	 *
	 * This function called outside guarded exit code, during all exits (including error exits).
	 */
	static void AppExit();

private:
	/** The first part of AppInit, which is shared with AppInitLoggingOnly. */
	static void AppInitUpToSetupOutputDevices();

	/** Utility function that processes Slate operations. */
	void ProcessLocalPlayerSlateOperations() const;

protected:

	/** Holds a dynamically expanding array of frame times in milliseconds (if FApp::IsBenchmarking() is set). */
	TArray<float> FrameTimes;

	/** Holds the total time spent ticking engine. */
	double TotalTickTime;
	
	/** Holds the maximum number of seconds engine should be ticked. */
	double MaxTickTime;
	
	/** Holds the maximum number of frames to render in benchmarking mode. */
	uint64 MaxFrameCounter;
	
	/** Holds the number of cycles in the last frame. */
	uint32 LastFrameCycles;

#if WITH_ENGINE

	/** Holds the objects which need to be cleaned up when the rendering thread finishes the previous frame. */
	FPendingCleanupObjects* PendingCleanupObjects;

#endif //WITH_ENGINE

private:


#if WITH_ENGINE

	/** Holds the engine service. */
	FEngineService* EngineService;

	/** Trace control service */
	FTraceService* TraceService;

	/** Holds the application session service. */
	TSharedPtr<ISessionService> SessionService;

#endif // WITH_ENGINE
	FPreInitContext PreInitContext;
};


/** Global engine loop object. This is needed so wxWindows can access it. */
extern FEngineLoop GEngineLoop;
