// Copyright Epic Games, Inc. All Rights Reserved.

#import <Foundation/Foundation.h>
#import <Cocoa/Cocoa.h>

@interface UEAppDelegate : NSObject <NSApplicationDelegate, NSFileManagerDelegate>
{
#if WITH_EDITOR
	NSString* Filename;
	bool bHasFinishedLaunching;
#endif
}

#if WITH_EDITOR
- (BOOL)application:(NSApplication *)theApplication openFile:(NSString *)filename;
#endif

@end

