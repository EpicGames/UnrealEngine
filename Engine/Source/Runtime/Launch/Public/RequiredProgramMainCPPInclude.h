// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LaunchEngineLoop.h"
#include "Misc/DirectHeaderCompiling.h"

#if !UE_DIRECT_HEADER_COMPILING(RequiredProgramMainCPPInclude)
// this is highly sketchy, but we need some stuff from launchengineloop.cpp
#include "Runtime/Launch/Private/LaunchEngineLoop.cpp"
#endif
